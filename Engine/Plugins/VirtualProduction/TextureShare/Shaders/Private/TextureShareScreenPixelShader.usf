// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

Texture2D InTexture;
SamplerState InTextureSampler;

float GammaModifier;

void Main(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
	)
{
	OutColor = Texture2DSample(InTexture, InTextureSampler, Input.UV);
	OutColor.rgb = pow( OutColor.rgb, GammaModifier);
}


