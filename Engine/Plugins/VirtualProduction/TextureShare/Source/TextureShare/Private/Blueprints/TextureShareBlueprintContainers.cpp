// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/TextureShareBlueprintContainers.h"
#include "Misc/TextureShareCoreStrings.h"
#include "UObject/Package.h"

namespace UE::TextureShare::BlueprintContainers
{
	static FString GetValidTextureShareObjectName(const FString& InShareName)
	{
		return InShareName.IsEmpty() ? UE::TextureShareCoreStrings::DefaultShareName : InShareName;
	}
};
using namespace UE::TextureShare::BlueprintContainers;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareObjectDesc
//////////////////////////////////////////////////////////////////////////////////////////////
FString FTextureShareObjectDesc::GetTextureShareObjectName() const
{
	return GetValidTextureShareObjectName(ShareName);
}

//////////////////////////////////////////////////////////////////////////////////////////////
// UTextureShareObject
//////////////////////////////////////////////////////////////////////////////////////////////
UTextureShareObject::UTextureShareObject()
{

}
UTextureShareObject::~UTextureShareObject()
{

}

void UTextureShareObject::SendCustomData(const TMap<FString, FString>& InSendParameters)
{
	CustomData.SendParameters.Empty();
	CustomData.SendParameters.Append(InSendParameters);
}

//////////////////////////////////////////////////////////////////////////////////////////////
// UTextureShare
//////////////////////////////////////////////////////////////////////////////////////////////
UTextureShare::UTextureShare()
{
	// Create default textureshare
	GetOrCreateTextureShareObject(UE::TextureShareCoreStrings::DefaultShareName);
}

UTextureShare::~UTextureShare()
{
	TextureShareObjects.Empty();
}

TSet<FString> UTextureShare::GetTextureShareObjectNames() const
{
	TSet<FString> Result;
	for (const TObjectPtr<UTextureShareObject>& TextureShareObjectIt : TextureShareObjects)
	{
		if (IsValid(TextureShareObjectIt) && TextureShareObjectIt->bEnable)
		{
			Result.Add(TextureShareObjectIt->Desc.GetTextureShareObjectName().ToLower());
		}
	}

	return Result;
}

UTextureShareObject* UTextureShare::GetTextureShareObject(const FString& InShareName) const
{
	TObjectPtr<UTextureShareObject> const* ExistObject = TextureShareObjects.FindByPredicate(
		[InShareName](const TObjectPtr<UTextureShareObject>& TextureShareObjectIt)
	{
		if (IsValid(TextureShareObjectIt) && TextureShareObjectIt->bEnable)
		{
			const FString ShareName = TextureShareObjectIt->Desc.GetTextureShareObjectName().ToLower();
			return ShareName == InShareName;
		}
		return false;
	});

	return ExistObject ? *ExistObject : nullptr;
}

const TArray<UTextureShareObject*> UTextureShare::GetTextureShareObjects() const
{
	TArray<UTextureShareObject*> OutTextureShareObjects;
	for (const TObjectPtr<UTextureShareObject>& TextureShareObjectIt : TextureShareObjects)
	{
		if (IsValid(TextureShareObjectIt))
		{
			OutTextureShareObjects.Add(TextureShareObjectIt.Get());
		}
	}

	return OutTextureShareObjects;
}

bool UTextureShare::RemoveTextureShareObject(const FString& InShareName)
{
	const FString ShareName = GetValidTextureShareObjectName(InShareName);

	const int32 Index = TextureShareObjects.IndexOfByPredicate(
		[ShareName](const TObjectPtr<UTextureShareObject>& ObjectIt)
		{
			return IsValid(ObjectIt) && ObjectIt->Desc.GetTextureShareObjectName() == ShareName;
		}
	);

	if (TextureShareObjects.IsValidIndex(Index) && IsValid(TextureShareObjects[Index]))
	{
		TextureShareObjects[Index]->ConditionalBeginDestroy();
		TextureShareObjects.RemoveAt(Index);

		return true;
	}

	return false;
}

UTextureShareObject* UTextureShare::GetOrCreateTextureShareObject(const FString& InShareName)
{
	const FString ShareName = GetValidTextureShareObjectName(InShareName);

	for(TObjectPtr<UTextureShareObject>& ObjectIt : TextureShareObjects)
	{
		if (IsValid(ObjectIt) && ObjectIt->Desc.GetTextureShareObjectName() == ShareName)
		{
			return ObjectIt;
		}
	}

	// Create default
	UTextureShareObject* NewTextureShareObject = NewObject<UTextureShareObject>(GetTransientPackage(), NAME_None, RF_Transient | RF_ArchetypeObject | RF_Public | RF_Transactional);
	if (NewTextureShareObject)
	{
		NewTextureShareObject->Desc.ShareName = InShareName;

		TextureShareObjects.Add(NewTextureShareObject);
	}

	return NewTextureShareObject;
}

