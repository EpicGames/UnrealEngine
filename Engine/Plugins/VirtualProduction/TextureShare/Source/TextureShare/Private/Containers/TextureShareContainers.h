// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Containers/TextureShareEnums.h"
#include "Containers/TextureShareContainers_Views.h"
#include "Containers/TextureShareContainers_Color.h"

/**
 * Data from TextureShareObject sent to proxy
 */
struct FTextureShareData
{
public:
	const FTextureShareCoreResourceRequest* FindResourceRequest(const FTextureShareCoreResourceDesc& InResourceDesc) const
	{
		for (const FTextureShareCoreObjectData& ObjectDataIt : ReceivedObjectsData)
		{
			if (const FTextureShareCoreResourceRequest* ExistResourceRequest = ObjectDataIt.Data.ResourceRequests.FindByEqualsFunc(InResourceDesc))
			{
				return ExistResourceRequest;
			}
		}

		return nullptr;
	}

public:
	// Map from UE stereoscopic pass to FTextureShareViewportDesc
	FTextureShareViewsData Views;

	// Saved local data from game thread
	FTextureShareCoreData ObjectData;

	// Saved received data from game thread
	TArray<FTextureShareCoreObjectData> ReceivedObjectsData;
};
