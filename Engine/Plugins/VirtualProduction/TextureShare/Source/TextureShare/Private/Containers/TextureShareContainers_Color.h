// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Containers/TextureShareEnums.h"

/** 
* This is a container with information related to color: gamma, sRGB, OCIO, etc.
* Used to support color conversion to and from the TS SDK.
*/
struct FTextureShareColorDesc
{
	/** Default constructor. */
	FTextureShareColorDesc() = default;

	/** Constructor for custom gamma. */
	FTextureShareColorDesc(const float InCustomGamma)
		: GammaType(ETextureShareResourceGammaType::Custom)
		, CustomGamma(InCustomGamma)
	{ }

	FTextureShareColorDesc(const ETextureShareResourceGammaType InGammaType)
		: GammaType(InGammaType)
	{ }

	/** Return true if gamma is defined.*/
	inline bool IsGammaDefined() const
	{
		switch (GammaType)
		{
		case ETextureShareResourceGammaType::Custom:
			return CustomGamma > 0.f;

		default:
			break;
		}

		return false;
	}

	/** Is the gamma should be converted. */
	inline bool ShouldConvertGamma(const FTextureShareColorDesc& InDestColor) const
	{
		return IsGammaDefined() && InDestColor.IsGammaDefined();
	}

	/** Implements compare operator. */
	inline bool operator==(const FTextureShareColorDesc& InDestColor) const
	{
		return GammaType == InDestColor.GammaType
			&& CustomGamma == InDestColor.CustomGamma;
	}

public:
	// Gamma type
	ETextureShareResourceGammaType GammaType = ETextureShareResourceGammaType::None;

	// User-defined gamma parameter. The default value of '-1' indicates undefined.
	float CustomGamma = -1.f;

	// sRGB, OCIO, etc. can be implemented here later.
};
