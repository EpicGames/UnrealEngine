// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Containers/TextureShareCoreContainers.h"

#include "StereoRendering.h"

/**
 * Scene view info: ViewDesc -> UE ViewIndex&Pass
 */
struct FTextureShareSceneViewInfo
{
	// View description
	FTextureShareCoreViewDesc ViewDesc;

	// View index
	int32 StereoViewIndex = INDEX_NONE;

	// Stereoscopic pass
	EStereoscopicPass StereoscopicPass = EStereoscopicPass::eSSP_PRIMARY;

public:
	FTextureShareSceneViewInfo() = default;

	FTextureShareSceneViewInfo(const FTextureShareCoreViewDesc& InViewDesc, const int32 InStereoViewIndex, const EStereoscopicPass InStereoscopicPass)
		: ViewDesc(InViewDesc), StereoViewIndex(InStereoViewIndex), StereoscopicPass(InStereoscopicPass)
	{ }

public:
	bool Equals(const int32 InStereoViewIndex, const EStereoscopicPass InStereoscopicPass) const
	{
		return StereoViewIndex == InStereoViewIndex && StereoscopicPass == InStereoscopicPass;
	}
};

/**
 * Map from UE stereoscopic pass to FTextureShareViewDesc
 */
struct FTextureShareViewsData
{
	TArray<FTextureShareSceneViewInfo> Views;

public:
	const FTextureShareSceneViewInfo* Find(const int32 InStereoViewIndex, const EStereoscopicPass InStereoscopicPass) const
	{
		return Views.FindByPredicate([InStereoViewIndex, InStereoscopicPass](const FTextureShareSceneViewInfo& ViewInfoIt)
		{
			return ViewInfoIt.Equals(InStereoViewIndex, InStereoscopicPass);
		});
	}

	bool Add(const FTextureShareCoreViewDesc& InViewDesc, const int32 InStereoViewIndex, const EStereoscopicPass InStereoscopicPass)
	{
		if (Find(InStereoViewIndex, InStereoscopicPass) == nullptr)
		{
			Views.Add(FTextureShareSceneViewInfo(InViewDesc, InStereoViewIndex, InStereoscopicPass));

			return true;
		}

		return false;
	}
};
