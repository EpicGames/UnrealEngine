// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

/**
* TextureShare proxy object flags
*/
enum class ETextureShareObjectProxyFlags : uint8
{
	None = 0,

	// The scene textures are writable. The other party can override UE scene textures.
	// This flag is used by the FTextureShareSceneViewExtension
	WritableSceneTextures = 1 << 0,

	// Session started (internal)
	SessionStarted = 1 << 5,

	// Frame proxy sync valid (internal)
	FrameProxySyncActive = 1 << 6,

	// The view extension is used by this TS proxy object (internal)
	// this means that callbacks will be called from the FTextureShareSceneViewExtension
	ViewExtensionUsed = 1 << 7,

	// Internal flags, that can't be override by the function SetObjectProxyFlags()
	InternalFlags
	= SessionStarted
	| FrameProxySyncActive
	| ViewExtensionUsed,
};
ENUM_CLASS_FLAGS(ETextureShareObjectProxyFlags);

/** 
* This enumeration defines the gamma conversion method implemented in TextureShare.
* 
* Texture resources in UE can use multiple gamma types.
* This can be a simple gamma based on the pow() function
* Or it can be a other gamma function that based on custom math (see /Engine/Private/GammaCorrectionCommon.ush)
* 
* The other party can request to share a resource with a custom gamut, such as linear, in order to process it.
* It is also expected that the result sent back will be converted to the original gamut.
*/
enum class ETextureShareResourceGammaType : uint8
{
	// Gamma is not used for this resource
	None = 0,

	// The gamma is based on pow() function.
	Custom,

	// Other types of gamut will be defined below.
};
