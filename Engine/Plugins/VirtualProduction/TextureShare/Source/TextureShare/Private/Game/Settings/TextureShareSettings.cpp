// Copyright Epic Games, Inc. All Rights Reserved.

#include "Game/Settings/TextureShareSettings.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"

namespace UE::TextureShare::GameSettings
{
	static constexpr auto GameConfigCategory = TEXT("/Script/Plugins.TextureShare");

	static const FString GetConfigDefaultGamePath()
	{
		static FString ConfigDefaultGamePath = FString::Printf(TEXT("%sDefaultGame.ini"), *FPaths::SourceConfigDir());
		return ConfigDefaultGamePath;
	}

	static void ClearReadOnlyForConfigDefaultGame()
	{
		if (FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*GetConfigDefaultGamePath()))
		{
			FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*GetConfigDefaultGamePath(), false);
		}
	}
};
using namespace UE::TextureShare::GameSettings;

//////////////////////////////////////////////////////////////////////////////////////////////
#define SAVE_SETTING_2ARG(SettingType, SettingName, Setting)\
	ClearReadOnlyForConfigDefaultGame();\
	GConfig->Set##SettingType(GameConfigCategory, TEXT(#SettingName), Setting, GetConfigDefaultGamePath());\
	bGConfigChanged = true;

#define SAVE_SETTING(SettingType, Setting) SAVE_SETTING_2ARG(SettingType, Setting, Setting)
#define SAVE_STRING_SETTING(Setting)       SAVE_SETTING_2ARG(String, Setting, *Setting)

#define LOAD_SETTING(SettingType, Setting) GConfig->Get##SettingType(GameConfigCategory, TEXT(#Setting), PluginSettings.Setting, GetConfigDefaultGamePath())

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareSettings
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareSettings FTextureShareSettings::GetSettings()
{
	FTextureShareSettings PluginSettings;
	{
		LOAD_SETTING(Bool,   bCreateDefaults);
		LOAD_SETTING(String, ProcessName);
	}

	return PluginSettings;
}

//////////////////////////////////////////////////////////////////////////////////////////////
const FName UTextureShareSettings::Container = TEXT("Project");
const FName UTextureShareSettings::Category  = TEXT("Plugins");
const FName UTextureShareSettings::Section   = TEXT("TextureShare");

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareDisplayCluster
//////////////////////////////////////////////////////////////////////////////////////////////
UTextureShareSettings::UTextureShareSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer) 
{
#if WITH_EDITOR
	GET_MEMBER_NAME_CHECKED(UTextureShareSettings, bCreateDefaults);
	GET_MEMBER_NAME_CHECKED(UTextureShareSettings, ProcessName);
#endif /*WITH_EDITOR*/
}

#if WITH_EDITOR
void UTextureShareSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.Property != nullptr)
	{
		bool bGConfigChanged = false;
		const FName PropertyName(PropertyChangedEvent.Property->GetFName());

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UTextureShareSettings, bCreateDefaults))
		{
			SAVE_SETTING(Bool, bCreateDefaults);
			
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTextureShareSettings, ProcessName))
		{
			SAVE_STRING_SETTING(ProcessName);
		}
		// ...

		if (bGConfigChanged)
		{
			GConfig->Flush(false, GetConfigDefaultGamePath());
		}
	}
}
#endif /*WITH_EDITOR*/
