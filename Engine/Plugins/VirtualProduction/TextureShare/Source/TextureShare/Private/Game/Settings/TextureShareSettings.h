// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"

#include "Misc/TextureShareCoreStrings.h"

#include "TextureShareSettings.generated.h"

/**
 * TextureShare plugin settings
 */
struct FTextureShareSettings
{
public:
	static FTextureShareSettings GetSettings();

public:
	// Enable base textureshare object creation by default
	bool bCreateDefaults = true;

	// Process name for this app
	FString ProcessName;
};

/**
 * TextureShare plugin settings
 **/
UCLASS(config = Engine, defaultconfig)
class TEXTURESHARE_API UTextureShareSettings : public UObject
{
	GENERATED_BODY()

public:
	UTextureShareSettings(const FObjectInitializer& ObjectInitializer);

public:
	static const FName Container;
	static const FName Category;
	static const FName Section;

#if WITH_EDITOR
protected:
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

#if WITH_EDITORONLY_DATA
protected:
	UPROPERTY(config, EditAnywhere, Category = Main, meta = (ToolTip = "Enable base textureshare object creation by default", ConfigRestartRequired = false))
	bool bCreateDefaults = true;

	UPROPERTY(config, EditAnywhere, Category = Main, meta = (ToolTip = "Process name for this app", ConfigRestartRequired = false))
	FString ProcessName = UE::TextureShareCoreStrings::DefaultProcessName::UE;
#endif /*WITH_EDITORONLY_DATA*/
};
