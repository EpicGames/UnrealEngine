// Copyright Epic Games, Inc. All Rights Reserved.

#include "Game/ViewExtension/TextureShareSceneViewExtension.h"

#include "Object/TextureShareObject.h"
#include "Object/TextureShareObjectProxy.h"

#include "Module/TextureShareLog.h"
#include "Misc/TextureShareStrings.h"

#include "ITextureShare.h"
#include "ITextureShareAPI.h"
#include "ITextureShareCallbacks.h"

#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RenderGraph.h"
#include "SceneRenderTargetParameters.h"
#include "SceneView.h"
#include "SceneRendering.h"
#include "SceneTextures.h"

namespace UE::TextureShare::SceneViewExtension
{
	/** Gets TextureShare module API. */
	static ITextureShareAPI& GetTextureShareAPI()
	{
		static ITextureShareAPI& TextureShareAPISingleton = ITextureShare::Get().GetTextureShareAPI();

		return TextureShareAPISingleton;
	}

	static void GetTextureShareCoreSceneViewMatrices(const FViewMatrices& InViewMatrices, FTextureShareCoreSceneViewMatrices& OutViewMatrices)
	{
		OutViewMatrices.ProjectionMatrix = InViewMatrices.GetViewToClip();
		OutViewMatrices.ProjectionNoAAMatrix = InViewMatrices.GetViewToClipNoAA();
		OutViewMatrices.ViewMatrix = InViewMatrices.GetWorldToView();
		OutViewMatrices.ViewProjectionMatrix = InViewMatrices.GetWorldToClip();
		OutViewMatrices.TranslatedViewProjectionMatrix = InViewMatrices.GetTranslatedWorldToClip();
		OutViewMatrices.PreViewTranslation = InViewMatrices.GetPreViewTranslation();
		OutViewMatrices.ViewOrigin = InViewMatrices.GetViewOrigin();
		OutViewMatrices.ProjectionScale = InViewMatrices.GetProjectionScale();
		OutViewMatrices.TemporalAAProjectionJitter = InViewMatrices.GetTemporalAAJitter();
		OutViewMatrices.ScreenScale = InViewMatrices.GetScreenScale();
	}

	static void GetTextureShareCoreSceneView(const FSceneView& InSceneView, FTextureShareCoreSceneView& OutSceneView)
	{
		GetTextureShareCoreSceneViewMatrices(InSceneView.ViewMatrices, OutSceneView.ViewMatrices);

		OutSceneView.UnscaledViewRect = InSceneView.UnscaledViewRect;
		OutSceneView.UnconstrainedViewRect = InSceneView.UnconstrainedViewRect;
		OutSceneView.ViewLocation = InSceneView.ViewLocation;
		OutSceneView.ViewRotation = InSceneView.ViewRotation;
		OutSceneView.BaseHmdOrientation = InSceneView.BaseHmdOrientation;
		OutSceneView.BaseHmdLocation = InSceneView.BaseHmdLocation;
		OutSceneView.WorldToMetersScale = InSceneView.WorldToMetersScale;

		OutSceneView.StereoViewIndex = InSceneView.StereoViewIndex;
		OutSceneView.PrimaryViewIndex = InSceneView.PrimaryViewIndex;

		OutSceneView.FOV = InSceneView.FOV;
		OutSceneView.DesiredFOV = InSceneView.DesiredFOV;
	}

	static void GetTextureShareCoreSceneGameTime(const FGameTime& InGameTime, FTextureShareCoreSceneGameTime& OutGameTime)
	{
		OutGameTime.RealTimeSeconds = InGameTime.GetRealTimeSeconds();
		OutGameTime.WorldTimeSeconds = InGameTime.GetWorldTimeSeconds();
		OutGameTime.DeltaRealTimeSeconds = InGameTime.GetDeltaRealTimeSeconds();
		OutGameTime.DeltaWorldTimeSeconds = InGameTime.GetDeltaWorldTimeSeconds();
	}

	static void GetTextureShareCoreSceneViewFamily(const FSceneViewFamily& InViewFamily, FTextureShareCoreSceneViewFamily& OutViewFamily)
	{
		GetTextureShareCoreSceneGameTime(InViewFamily.Time, OutViewFamily.GameTime);

		OutViewFamily.FrameNumber = InViewFamily.FrameNumber;
		OutViewFamily.bIsHDR = InViewFamily.bIsHDR;
		OutViewFamily.SecondaryViewFraction = InViewFamily.SecondaryViewFraction;
	}
};

FTextureShareSceneView::FTextureShareSceneView(const FSceneViewFamily& InViewFamily, const FSceneView& InSceneView, const FTextureShareSceneViewInfo& InViewInfo)
	: ViewInfo(InViewInfo)
	, SceneView(InSceneView)
{
#if WITH_MGPU
	checkSlow(InSceneView.bIsViewInfo);
	const FViewInfo& InSceneViewInfo = static_cast<const FViewInfo&>(InSceneView);
	const uint32 GPUIndex_RenderingThread = InSceneViewInfo.GPUMask.GetFirstIndex();

	GPUIndex = (GPUIndex_RenderingThread < GNumExplicitGPUsForRendering) ? GPUIndex_RenderingThread : -1;
#endif

	UnconstrainedViewRect = SceneView.UnconstrainedViewRect;
	UnscaledViewRect = SceneView.UnscaledViewRect;
};

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareSceneViewExtension
//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareSceneViewExtension::GetSceneViewData_RenderThread(const FTextureShareSceneView& InView, ITextureShareObjectProxy& ObjectProxy)
{
	if (!bUseThisViewExtension)
	{
		// Ignore unused VE
		return;
	}

	using namespace UE::TextureShare::SceneViewExtension;

	const FSceneView& InSceneView = InView.SceneView;
	if (InSceneView.Family)
	{
		const FSceneViewFamily& InViewFamily = *InSceneView.Family;

		// Create new data container for viewport eye
		FTextureShareCoreSceneViewData SceneViewData(InView.ViewInfo.ViewDesc);

		// Get view eye data
		GetTextureShareCoreSceneView(InSceneView, SceneViewData.View);
		GetTextureShareCoreSceneViewFamily(InViewFamily, SceneViewData.ViewFamily);

		// Save scene viewport eye data
		TArraySerializable<FTextureShareCoreSceneViewData>& DstSceneData = ObjectProxy.GetCoreProxyData_RenderThread().SceneData;
		if (FTextureShareCoreSceneViewData* ExistValue = DstSceneData.FindByEqualsFunc(SceneViewData.ViewDesc))
		{
			*ExistValue = SceneViewData;
		}
		else
		{
			DstSceneData.Add(SceneViewData);
		}
	}
}

void FTextureShareSceneViewExtension::ShareSceneViewColors_RenderThread(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures, const FTextureShareSceneView& InView) const
{
	if (!bUseThisViewExtension)
	{
		// Ignore unused VE
		return;
	}

	using namespace UE::TextureShare::SceneViewExtension;
	TSharedPtr<ITextureShareObjectProxy, ESPMode::ThreadSafe> ObjectProxy = GetTextureShareAPI().GetObjectProxy_RenderThread(SharedThis(this));
	if (!ObjectProxy.IsValid())
	{
		return;
	}

	const auto AddShareTexturePass = [&](
		const TCHAR* InTextureName,
		const FRDGTextureRef& InTextureRef,
		const FTextureShareColorDesc& InColorDesc,
		const FIntRect* CustomSrcRect = nullptr)
	{
		// Send resource
		ObjectProxy->ShareResource_RenderThread(
			GraphBuilder,
			FTextureShareCoreResourceDesc(InTextureName, InView.ViewInfo.ViewDesc, ETextureShareTextureOp::Read),
			InTextureRef,
			InColorDesc,
			InView.GPUIndex,
			CustomSrcRect ? CustomSrcRect : &InView.UnconstrainedViewRect);
	};

	// Scene colors textures is in linear space
	const float SceneColorsGamma = 1.0f;
	// Gathering UE texture color information
	const FTextureShareColorDesc SceneColorDesc(SceneColorsGamma);

	// For textures that do not contain color information
	const FTextureShareColorDesc DataColorDesc(ETextureShareResourceGammaType::None);

	AddShareTexturePass(UE::TextureShareStrings::SceneTextures::SceneColor, SceneTextures.Color.Resolve, SceneColorDesc);

	AddShareTexturePass(UE::TextureShareStrings::SceneTextures::SceneDepth, SceneTextures.Depth.Resolve, DataColorDesc);
	
	// Small depth use downscale size
	const FIntRect SmallDepthRect = GetDownscaledRect(InView.UnconstrainedViewRect, SceneTextures.Config.SmallDepthDownsampleFactor);
	AddShareTexturePass(UE::TextureShareStrings::SceneTextures::SmallDepthZ, SceneTextures.SmallDepth, DataColorDesc, &SmallDepthRect);

	AddShareTexturePass(UE::TextureShareStrings::SceneTextures::GBufferA, SceneTextures.GBufferA, DataColorDesc);
	AddShareTexturePass(UE::TextureShareStrings::SceneTextures::GBufferB, SceneTextures.GBufferB, DataColorDesc);
	AddShareTexturePass(UE::TextureShareStrings::SceneTextures::GBufferC, SceneTextures.GBufferC, DataColorDesc);
	AddShareTexturePass(UE::TextureShareStrings::SceneTextures::GBufferD, SceneTextures.GBufferD, DataColorDesc);
	AddShareTexturePass(UE::TextureShareStrings::SceneTextures::GBufferE, SceneTextures.GBufferE, DataColorDesc);
	AddShareTexturePass(UE::TextureShareStrings::SceneTextures::GBufferF, SceneTextures.GBufferF, DataColorDesc);
}

//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareSceneViewExtension::FTextureShareSceneViewExtension(const FAutoRegister& AutoRegister, FViewport* InLinkedViewport)
	: FSceneViewExtensionBase(AutoRegister)
	, LinkedViewport(InLinkedViewport)
{ }

FTextureShareSceneViewExtension::~FTextureShareSceneViewExtension()
{ }

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareSceneViewExtension::IsStereoRenderingAllowed() const
{
	return LinkedViewport && LinkedViewport->IsStereoRenderingAllowed();
}

bool FTextureShareSceneViewExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const
{
	return bUseThisViewExtension && (LinkedViewport == Context.Viewport);
}

bool FTextureShareSceneViewExtension::IsEnabled_RenderThread() const
{
	if (Views.IsEmpty())
	{
		// Ignore another ViewFamilies
		return false;
	}

	if (!bUseThisViewExtension)
	{
		// Ignore unused VE
		return false;
	}

	return true;
}

void FTextureShareSceneViewExtension::BeginRenderViewFamily(FSceneViewFamily& InViewFamily)
{
	if (!bUseThisViewExtension)
	{
		// Ignore unused VE
		return;
	}

	using namespace UE::TextureShare::SceneViewExtension;
	TSharedPtr<ITextureShareObject, ESPMode::ThreadSafe> Object = GetTextureShareAPI().GetObject(SharedThis(this));
	if (!Object.IsValid())
	{
		return;
	}

	// Perform callback logic
	if (GetTextureShareAPI().GetCallbacks().OnTextureShareBeginRenderViewFamily().IsBound())
	{
		GetTextureShareAPI().GetCallbacks().OnTextureShareBeginRenderViewFamily().Broadcast(InViewFamily, *Object);
	}
}

void FTextureShareSceneViewExtension::PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily)
{
	if (!bUseThisViewExtension)
	{
		// Ignore unused VE
		return;
	}

	// Reset values
	Views.Empty();

	using namespace UE::TextureShare::SceneViewExtension;
	TSharedPtr<ITextureShareObjectProxy, ESPMode::ThreadSafe> ObjectProxy = GetTextureShareAPI().GetObjectProxy_RenderThread(SharedThis(this));
	if (!ObjectProxy.IsValid())
	{
		return;
	}

	// Initialize views
	for (const FSceneView* SceneViewIt : InViewFamily.Views)
	{
		if (SceneViewIt)
		{
			if (const FTextureShareSceneViewInfo* ViewInfo = ObjectProxy->GetData_RenderThread().Views.Find(SceneViewIt->StereoViewIndex, SceneViewIt->StereoPass))
			{
				Views.Add(FTextureShareSceneView(InViewFamily, *SceneViewIt, *ViewInfo));
			}
		}
	}

	if (!IsEnabled_RenderThread())
	{
		return;
	}

	// Add RDG pass
	AddPass(GraphBuilder, RDG_EVENT_NAME("PreRenderViewFamily_RenderThread"), [this, &InViewFamily](FRHICommandListImmediate& RHICmdList)
		{
			PreRenderViewFamily_RenderThread(RHICmdList, InViewFamily);
		});
}

void FTextureShareSceneViewExtension::PostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily)
{
	if (!IsEnabled_RenderThread())
	{
		return;
	}

	AddPass(GraphBuilder, RDG_EVENT_NAME("PostRenderViewFamily_RenderThread"), [this, &InViewFamily](FRHICommandListImmediate& RHICmdList)
		{
			PostRenderViewFamily_RenderThread(RHICmdList, InViewFamily);
		});
}

void FTextureShareSceneViewExtension::PreRenderViewFamily_RenderThread(FRHICommandListImmediate& RHICmdList, FSceneViewFamily& InViewFamily)
{
	if (!IsEnabled_RenderThread())
	{
		return;
	}

	using namespace UE::TextureShare::SceneViewExtension;
	TSharedPtr<ITextureShareObjectProxy, ESPMode::ThreadSafe> ObjectProxy = GetTextureShareAPI().GetObjectProxy_RenderThread(SharedThis(this));
	if (!ObjectProxy.IsValid())
	{
		return;
	}

	// Perform callback logic
	if (GetTextureShareAPI().GetCallbacks().OnTextureSharePreRenderViewFamily_RenderThread().IsBound())
	{
		GetTextureShareAPI().GetCallbacks().OnTextureSharePreRenderViewFamily_RenderThread().Broadcast(RHICmdList, *ObjectProxy);
	}
}

void FTextureShareSceneViewExtension::PostRenderViewFamily_RenderThread(FRHICommandListImmediate& RHICmdList, FSceneViewFamily& InViewFamily)
{
	if (!IsEnabled_RenderThread())
	{
		return;
	}

	using namespace UE::TextureShare::SceneViewExtension;
	TSharedPtr<ITextureShareObjectProxy, ESPMode::ThreadSafe> ObjectProxy = GetTextureShareAPI().GetObjectProxy_RenderThread(SharedThis(this));
	if (!ObjectProxy.IsValid())
	{
		return;
	}

	if (!InViewFamily.RenderTarget)
	{
		return;
	}

	// User must set this flag to make scene textures writable
	const bool bEnableReceiving = EnumHasAnyFlags(ObjectProxy->GetObjectProxyFlags(), ETextureShareObjectProxyFlags::WritableSceneTextures);

	bool bUseFrameSceneFinalColorEnd = false;

	for (const FTextureShareSceneView& ViewIt : Views)
	{
		// Always get scene view data
		GetSceneViewData_RenderThread(ViewIt, *ObjectProxy);

		// Share only if the resource is requested from a remote process
		if (const FTextureShareCoreResourceRequest* ExistResourceRequest = ObjectProxy->GetData_RenderThread().FindResourceRequest(FTextureShareCoreResourceDesc(UE::TextureShareStrings::SceneTextures::FinalColor, ViewIt.ViewInfo.ViewDesc, ETextureShareTextureOp::Undefined)))
		{
			FTextureRHIRef RenderTargetTexture = InViewFamily.RenderTarget->GetRenderTargetTexture();
			if (RenderTargetTexture.IsValid())
			{
				const float DisplayGamma = InViewFamily.RenderTarget->GetDisplayGamma();

				// Gathering UE texture color information
				const FTextureShareColorDesc UEResourceColorDesc(DisplayGamma);

				// Send
				const FTextureShareCoreResourceDesc SendResourceDesc(UE::TextureShareStrings::SceneTextures::FinalColor, ViewIt.ViewInfo.ViewDesc, ETextureShareTextureOp::Read);
				
				ObjectProxy->ShareResource_RenderThread(
					RHICmdList,
					SendResourceDesc,
					RenderTargetTexture,
					UEResourceColorDesc,
					ViewIt.GPUIndex,
					&ViewIt.UnscaledViewRect);

				if (bEnableReceiving)
				{
					// Receive
					const FTextureShareCoreResourceDesc ReceiveResourceDesc(UE::TextureShareStrings::SceneTextures::FinalColor, ViewIt.ViewInfo.ViewDesc, ETextureShareTextureOp::Write, ETextureShareSyncStep::FrameSceneFinalColorEnd);
					if (ObjectProxy->ShareResource_RenderThread(
						RHICmdList,
						ReceiveResourceDesc,
						RenderTargetTexture,
						UEResourceColorDesc,
						ViewIt.GPUIndex,
						&ViewIt.UnscaledViewRect))
					{
						bUseFrameSceneFinalColorEnd = true;
					}
				}
			}
		}
	}

	if (bUseFrameSceneFinalColorEnd)
	{
		ObjectProxy->FrameSync_RenderThread(RHICmdList, ETextureShareSyncStep::FrameSceneFinalColorEnd);
	}

	// Perform callback logic
	if (GetTextureShareAPI().GetCallbacks().OnTextureSharePostRenderViewFamily_RenderThread().IsBound())
	{
		GetTextureShareAPI().GetCallbacks().OnTextureSharePostRenderViewFamily_RenderThread().Broadcast(RHICmdList, *ObjectProxy);
	}
}

void FTextureShareSceneViewExtension::OnResolvedSceneColor_RenderThread(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures) const
{
	if (!IsEnabled_RenderThread())
	{
		return;
	}

	// Send scene textures on request
	for (const FTextureShareSceneView& ViewIt : Views)
	{
		ShareSceneViewColors_RenderThread(GraphBuilder, SceneTextures, ViewIt);
	}
}
