// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "SceneViewExtension.h"
#include "Containers/TextureShareCoreEnums.h"
#include "Containers/TextureShareContainers_Views.h"

#include "Delegates/DelegateCombinations.h"

class ITextureShareObjectProxy;
class SWindow;
class FSceneViewFamily;
class FSceneView;
class FViewport;
struct FSceneTextures;
class FRDGBuilder;
class FRHICommandListImmediate;

/** A container with a saved scene view for TS. */
struct FTextureShareSceneView
{
	FTextureShareSceneView(const FSceneViewFamily& InViewFamily, const FSceneView& InSceneView, const FTextureShareSceneViewInfo& InViewInfo);

public:
	int32 GPUIndex = -1;

	FIntRect UnconstrainedViewRect;
	FIntRect UnscaledViewRect;

	FTextureShareSceneViewInfo ViewInfo;

	const FSceneView& SceneView;
};

/**
 * A view extension to handle a multi-threaded renderer for a TextureShare object.
 */
class TEXTURESHARE_API FTextureShareSceneViewExtension : public FSceneViewExtensionBase
{
private:
	// A quick and dirty way to determine which TS ViewExtension (sub)class this is. Every subclass should implement it.
	virtual FName GetRTTI() const { return TEXT("FTextureShareSceneViewExtension"); }

public:
	FTextureShareSceneViewExtension(const FAutoRegister&, FViewport* InLinkedViewport);
	virtual ~FTextureShareSceneViewExtension();

public:
	//~ Begin ISceneViewExtension interface
	virtual int32 GetPriority() const override { return -1; }

	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;

	virtual void PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) override;
	virtual void PostRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) override;

	virtual void OnResolvedSceneColor_RenderThread(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures) const;

	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;
	//~End ISceneViewExtension interface

public:
	virtual void GetSceneViewData_RenderThread(const FTextureShareSceneView& InView, ITextureShareObjectProxy& ObjectProxy);
	virtual void ShareSceneViewColors_RenderThread(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures, const FTextureShareSceneView& InView) const;

public:
	// Returns true if the given object is of the same type.
	bool IsA(const FTextureShareSceneViewExtension&& Other) const
	{
		return GetRTTI() == Other.GetRTTI();
	}

	bool IsStereoRenderingAllowed() const;

	/** Marks this VE as unused. */
	void Release_RenderThread()
	{
		bUseThisViewExtension = false;
	}

private:
	void PreRenderViewFamily_RenderThread(FRHICommandListImmediate& RHICmdList, FSceneViewFamily& InViewFamily);
	void PostRenderViewFamily_RenderThread(FRHICommandListImmediate& RHICmdList, FSceneViewFamily& InViewFamily);

	/** true, if this VE can be used. */
	bool IsEnabled_RenderThread() const;

public:
	/** Viewport to which we are attached */
	FViewport* const LinkedViewport;

protected:
	// Internal collection of used views
	TArray<FTextureShareSceneView> Views;

	/** Is this VE used. */
	bool bUseThisViewExtension = true;
};
