// Copyright Epic Games, Inc. All Rights Reserved.

#include "Game/WorldSubsystem/TextureShareWorldSubsystem.h"
#include "Game/WorldSubsystem/TextureShareWorldSubsystemContext.h"
#include "Game/WorldSubsystem/TextureShareWorldSubsystemProxy.h"
#include "Game/Settings/TextureShareSettings.h"

#include "Module/TextureShareLog.h"

#include "ITextureShare.h"
#include "ITextureShareAPI.h"
#include "ITextureShareObject.h"

#include "Engine/World.h"
#include "Engine/GameViewportClient.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/Package.h"

// This CVar enables/disables the WorldSubsystem TS object type.
static TAutoConsoleVariable<int32> CVarTextureShareUseWorldSubsystem(
	TEXT("TextureShare.Enable.WorldSubsystem"),
	1,
	TEXT("Enable world subsystems objects (0 = disabled)\n"),
	ECVF_RenderThreadSafe
);

namespace UE::TextureShare::WorldSubsystem
{
	static ITextureShareAPI& TextureShareAPI()
	{
		static ITextureShareAPI& TextureShareAPISingletone = ITextureShare::Get().GetTextureShareAPI();
		return TextureShareAPISingletone;
	}
};
using namespace UE::TextureShare::WorldSubsystem;

//////////////////////////////////////////////////////////////////////////////////////////////
// UTextureShareWorldSubsystem
//////////////////////////////////////////////////////////////////////////////////////////////
UTextureShareWorldSubsystem::UTextureShareWorldSubsystem()
{
	FTextureShareSettings PluginSettings = FTextureShareSettings::GetSettings();
	if(PluginSettings.bCreateDefaults)
	{
		// Create default
		TextureShare = NewObject<UTextureShare>(GetTransientPackage(), NAME_None, RF_Transient | RF_ArchetypeObject | RF_Public | RF_Transactional);
	}
}

UTextureShareWorldSubsystem::~UTextureShareWorldSubsystem()
{
	TextureShare = nullptr;
}

//////////////////////////////////////////////////////////////////////////////////////////////
TStatId UTextureShareWorldSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UTextureShareWorldSubsystem, STATGROUP_Tickables);
}

void UTextureShareWorldSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
}

void UTextureShareWorldSubsystem::Deinitialize()
{
	if (UWorld* World = GetWorld())
	{
		OnWorldEndPlay(*World);
	}

	Super::Deinitialize();
}

//////////////////////////////////////////////////////////////////////////////////////////////
void UTextureShareWorldSubsystem::Release()
{
	// Remove all created TextureShare objects
	for (const FString& TextureShareNameIt : NamesOfExistingObjects)
	{
		if (!TextureShareNameIt.IsEmpty())
		{
			TextureShareAPI().RemoveObject(TextureShareNameIt);
		}
	}

	NamesOfExistingObjects.Empty();
}

//////////////////////////////////////////////////////////////////////////////////////////////
void UTextureShareWorldSubsystem::OnWorldBeginPlay(UWorld& InWorld)
{
	bWorldPlay = true;

	UE_TS_LOG(LogTextureShareWorldSubsystem, Verbose, TEXT("OnWorldBeginPlay"));

	FTextureShareSettings PluginSettings = FTextureShareSettings::GetSettings();
	if (!PluginSettings.ProcessName.IsEmpty())
	{
		TextureShareAPI().SetProcessName(PluginSettings.ProcessName);
	}

	TextureShareAPI().OnWorldBeginPlay(InWorld);

	Super::OnWorldBeginPlay(InWorld);
}

void UTextureShareWorldSubsystem::OnWorldEndPlay(UWorld& InWorld)
{
	if (bWorldPlay)
	{
		UE_TS_LOG(LogTextureShareWorldSubsystem, Verbose, TEXT("OnWorldEndPlay"));

		TextureShareAPI().OnWorldEndPlay(InWorld);

		bWorldPlay = false;
		Release();
	}
}

void UTextureShareWorldSubsystem::Tick(float DeltaTime)
{
	if (bWorldPlay)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TextureShare::WorldSubsystem::Tick);

		if (CVarTextureShareUseWorldSubsystem.GetValueOnGameThread() > 0)
		{
			if (TextureShare && TextureShare->IsEnabled())
			{
				// Update the list of used object names for the current frame
				// and remove unused objects
				const TSet<FString> PrevFrameObjectNames(NamesOfExistingObjects);
				NamesOfExistingObjects = TextureShare->GetTextureShareObjectNames();


				for (const FString& ShareNameIt : PrevFrameObjectNames)
				{
					if (NamesOfExistingObjects.Contains(ShareNameIt) == false)
					{
						if (!ShareNameIt.IsEmpty())
						{
							TextureShareAPI().RemoveObject(ShareNameIt);
						}
					}
				}
			}

			// GetOrCreate and Tick existing objects
			if (UWorld* World = GetWorld())
			{
				if (UGameViewportClient* GameViewportClient = World->GetGameViewport())
				{
					if (FViewport* DstViewport = GameViewportClient->Viewport)
					{
						// Update existing objects
						for (const FString& ShareNameIt : NamesOfExistingObjects)
						{
							if (UTextureShareObject* BPTextureShareObject = TextureShare->GetTextureShareObject(ShareNameIt))
							{
								static ITextureShareAPI& TextureShareAPI = ITextureShare::Get().GetTextureShareAPI();
								const TSharedPtr<ITextureShareObject, ESPMode::ThreadSafe> TextureShareObject = TextureShareAPI.GetOrCreateObject(
									BPTextureShareObject->Desc.GetTextureShareObjectName());

								if(TextureShareObject.IsValid())
								{
									// Update process name (empty or equal values will be ignored)
									TextureShareObject->SetProcessId(TextureShare->ProcessName);

									// Update TS object sync settings
									const FTextureShareObjectSyncSettings InBPSyncSettings = BPTextureShareObject->Desc.Settings;
									{
										FTextureShareCoreSyncSettings InOutSyncSetting = TextureShareObject->GetSyncSetting();

										InOutSyncSetting.TimeoutSettings.FrameBeginTimeOut = InBPSyncSettings.FrameConnectTimeOut;
										InOutSyncSetting.TimeoutSettings.FrameSyncTimeOut = InBPSyncSettings.FrameSyncTimeOut;

										InOutSyncSetting.FrameSyncSettings = TextureShareObject->GetFrameSyncSettings(ETextureShareFrameSyncTemplate::Default);

										TextureShareObject->SetSyncSetting(InOutSyncSetting);
									}

									// Create new TS context:
									TSharedRef<FTextureShareWorldSubsystemContext, ESPMode::ThreadSafe> TextureShareContext =
										MakeShared<FTextureShareWorldSubsystemContext, ESPMode::ThreadSafe>();

									// Configure context:
									{
										// Get any possible send resources
										for (const FTextureShareSendTextureDesc& SendTextureDescIt : BPTextureShareObject->Textures.SendTextures)
										{
											FTextureShareWorldSubsystemTextureProxy SendTextureProxy(SendTextureDescIt);
											if(SendTextureProxy.IsEnabled())
											{
												TextureShareContext->Send.Emplace(SendTextureDescIt.Name, SendTextureProxy);
											}
										}

										// and receive
										for (const FTextureShareReceiveTextureDesc& ReceiveTextureDescIt : BPTextureShareObject->Textures.ReceiveTextures)
										{
											FTextureShareWorldSubsystemRenderTargetResourceProxy ReceiveRTTProxy(ReceiveTextureDescIt);
											if (ReceiveRTTProxy.IsEnabled())
											{
												TextureShareContext->Receive.Emplace(ReceiveTextureDescIt.Name, ReceiveRTTProxy);
											}
										}
									}

									// Sets a new TS context
									TextureShareObject->SetTextureShareContext(TextureShareContext);

									// Tick
									TextureShareContext->Tick(*TextureShareObject, *BPTextureShareObject, DstViewport);
								}
							}
						}
					}
				}
			}
		}
	}

	Super::Tick(DeltaTime);
}
