// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"

#include "Subsystems/WorldSubsystem.h"

#include "Blueprints/TextureShareBlueprintContainers.h"

#include "TextureShareWorldSubsystem.generated.h"

/**
 * Tickable TextureShare World Subsystem used to handle tick and react to level and world changes.
 */
UCLASS(BlueprintType, Transient)
class TEXTURESHARE_API UTextureShareWorldSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()

public:
	UTextureShareWorldSubsystem();
	virtual ~UTextureShareWorldSubsystem();

public:
	virtual void OnWorldBeginPlay(UWorld& InWorld) override;
	virtual void OnWorldEndPlay(UWorld& InWorld);

public:
	//~FTickableGameObject implementation Begin
	virtual bool IsTickable() const override { return true; }
	virtual bool IsTickableInEditor() const override { return false; }
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;
	//~~FTickableGameObject implementation End

public:
	//~USubsystem implementation Begin
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~~USubsystem implementation End

public:
	// Get textureshare API UObject
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "TextureShareAPI"), Category = "TextureShare")
	UTextureShare* GetTextureShare() const
	{
		return TextureShare;
	}

public:
	// This UObject implements configuration and API
	UPROPERTY(Transient)
	TObjectPtr<UTextureShare> TextureShare;

private:
	// Release objects and resources
	void Release();

private:
	// Names of the created TextureShare objects
	TSet<FString> NamesOfExistingObjects;

	// Handle logic OnWorldBeginPlay/OnWorldEndPlay
	bool bWorldPlay = false;
};
