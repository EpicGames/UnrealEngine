// Copyright Epic Games, Inc. All Rights Reserved.

#include "Game/WorldSubsystem/TextureShareWorldSubsystemContext.h"

#include "Containers/TextureShareContainers.h"
#include "Blueprints/TextureShareBlueprintContainers.h"

#include "Module/TextureShareLog.h"
#include "Misc/TextureShareStrings.h"

#include "ITextureShare.h"
#include "ITextureShareAPI.h"
#include "ITextureShareObject.h"
#include "ITextureShareObjectProxy.h"
#include "ITextureShareCallbacks.h"

#include "TextureResource.h"
#include "SceneView.h"
#include "Misc/ScopeLock.h"

namespace UE::TextureShareWorldSubsystemContext
{
	/** Gets TextureShare module API. */
	static ITextureShareAPI& GetTextureShareAPI()
	{
		static ITextureShareAPI& TextureShareAPISingleton = ITextureShare::Get().GetTextureShareAPI();

		return TextureShareAPISingleton;
	}

	static float GetBackbufferGamma()
	{
		// The gamma from the project settings should be used.
		return 2.2f;
	}

	/** StereoView descriptor*/
	struct FStereoViewDesc
	{
		FStereoViewDesc(const int32 InStereoViewIndex, const EStereoscopicPass InStereoscopicPass)
			: StereoViewIndex(InStereoViewIndex), StereoscopicPass(InStereoscopicPass)
		{ }

		inline bool operator==(const FStereoViewDesc& In) const
		{
			return StereoViewIndex == In.StereoViewIndex
				&& StereoscopicPass == In.StereoscopicPass;
		}

		// Stereo view index
		int32 StereoViewIndex = INDEX_NONE;

		// Stereoscopic pass
		EStereoscopicPass StereoscopicPass = EStereoscopicPass::eSSP_FULL;
	};

	/**
	* Returns the mapping of supported eyes to stereo views.
	*/
	static const TMap<ETextureShareEyeType, FStereoViewDesc>& GetSupportedStereoViews()
	{
		static TMap<ETextureShareEyeType, FStereoViewDesc> SupportedStereoViews;
		if (SupportedStereoViews.IsEmpty())
		{
			// Initialize
			SupportedStereoViews.Emplace(ETextureShareEyeType::Default, FStereoViewDesc(INDEX_NONE, EStereoscopicPass::eSSP_FULL));
			SupportedStereoViews.Emplace(ETextureShareEyeType::StereoLeft,  FStereoViewDesc(0, EStereoscopicPass::eSSP_PRIMARY));
			SupportedStereoViews.Emplace(ETextureShareEyeType::StereoRight, FStereoViewDesc(1, EStereoscopicPass::eSSP_SECONDARY));
		}

		return SupportedStereoViews;
	}

	/**
	* Check if this view family supported
	*/
	static bool IsViewFamilySupported(const FSceneViewFamily& InViewFamily)
	{
		for (const FSceneView* SceneViewIt : InViewFamily.Views)
		{
			if (SceneViewIt)
			{
				if (const ETextureShareEyeType* EyeType = GetSupportedStereoViews().FindKey(
					FStereoViewDesc(SceneViewIt->StereoViewIndex, SceneViewIt->StereoPass)))
				{
					// This view can be mapped by this TS object
					return true;
				}
			}
		}
		

		return false;
	}
};

void FTextureShareWorldSubsystemContext::Tick_GameThread(ITextureShareObject& TextureShareObject)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(TextureShare::WorldSubsystemContext::Tick_GameThread);

	using namespace UE::TextureShareWorldSubsystemContext;
	if (TextureShareObject.BeginFrameSync())
	{
		// Update frame marker for current frame
		TextureShareObject.GetCoreData().FrameMarker.NextFrame();

		FTextureShareCustomData* CustomData = IsValid(TextureShareUObject.Get()) ? &TextureShareUObject->CustomData : nullptr;

		// Send Custom Data
		if (CustomData && !CustomData->SendParameters.IsEmpty())
		{
			for (const TPair<FString, FString>& ParamIt : CustomData->SendParameters)
			{
				TextureShareObject.GetCoreData().CustomData.Add(FTextureShareCoreCustomData(ParamIt.Key, ParamIt.Value));
			}
		}

		if (TextureShareObject.FrameSync(ETextureShareSyncStep::FramePreSetupBegin))
		{
			// Process resources requests.
			{
				for (const FTextureShareCoreObjectData& ObjectDataIt : TextureShareObject.GetReceivedCoreObjectData())
				{
					for (const FTextureShareCoreResourceRequest& RequestIt : ObjectDataIt.Data.ResourceRequests)
					{
						// Add mapping on UE rendering
						const FTextureShareCoreViewDesc& ViewDesc = RequestIt.ResourceDesc.ViewDesc;
						if (const FStereoViewDesc* StereoViewDesc = GetSupportedStereoViews().Find(ViewDesc.EyeType))
						{
							TextureShareObject.GetData().Views.Add(ViewDesc, StereoViewDesc->StereoViewIndex, StereoViewDesc->StereoscopicPass);
						}
					}
				}
			}

			// Receive Custom Data
			if (CustomData)
			{
				CustomData->ReceivedParameters.Empty();
				for (const FTextureShareCoreObjectData& ObjectDataIt : TextureShareObject.GetReceivedCoreObjectData())
				{
					for (const FTextureShareCoreCustomData& ParamIt : ObjectDataIt.Data.CustomData)
					{
						CustomData->ReceivedParameters.Emplace(ParamIt.Key, ParamIt.Value);
					}
				}
			}
		}

		bGameThreadSynchronized = TextureShareObject.EndFrameSync();

		UE_TS_LOG(LogTextureShareWorldSubsystem, Log, TEXT("%s:Tick_GameThread() %s"),
			*TextureShareObject.GetName(),
			bGameThreadSynchronized ? TEXT("successfully completed") : TEXT("Failed."));
	}
}

void FTextureShareWorldSubsystemContext::Tick(
	ITextureShareObject& Object,
	UTextureShareObject& InOutTextureShareUObject,
	FViewport* InViewport)
{
	using namespace UE::TextureShareWorldSubsystemContext;

	check(IsInGameThread());

	// Save ptr on customn data UObject
	TextureShareUObject = &InOutTextureShareUObject;

	Object.BeginSession(InViewport);

	// Enable receive for scene textures (single viewport case)
	Object.GetProxy()->SetObjectProxyFlags(Object.GetProxy()->GetObjectProxyFlags() | ETextureShareObjectProxyFlags::WritableSceneTextures);

	// Other logic will be called from the VE in the OnTextureShareBeginRenderViewFamily() callback
};

bool FTextureShareWorldSubsystemContext::ShouldUseBackbufferTexture(const ITextureShareObjectProxy& ObjectProxy) const
{
	// Find any backbuffer resource request
	const FTextureShareCoreResourceRequest* AnyBackbufferResourceRequest = ObjectProxy.GetData_RenderThread().FindResourceRequest(
		FTextureShareCoreResourceDesc(UE::TextureShareStrings::SceneTextures::Backbuffer, ETextureShareTextureOp::Undefined));

	return AnyBackbufferResourceRequest != nullptr;
}

void FTextureShareWorldSubsystemContext::ShareResources_RenderThread(FRHICommandListImmediate& RHICmdList, const ITextureShareObjectProxy& ObjectProxy) const
{
	if (!ObjectProxy.IsFrameSyncActive_RenderThread())
	{
		return;
	}

	// mgpu is currently not handled by this logic
	const int32 GPUIndex = -1;

	// Send custom textures
	for (const TPair<FString, FTextureShareWorldSubsystemTextureProxy>& SendIt : Send)
	{
		// Share only requested resources
		if (const FTextureShareCoreResourceRequest* ExistResourceRequest = ObjectProxy.GetData_RenderThread().FindResourceRequest(
			FTextureShareCoreResourceDesc(
				SendIt.Key.ToLower(),
				ETextureShareTextureOp::Read))
			)
		{
			if (FRHITexture* RHITextureToSend = SendIt.Value.Texture ? SendIt.Value.Texture->GetTexture2DRHI() : nullptr)
			{
				// Send if remote process request to read this texture
				ObjectProxy.ShareResource_RenderThread(
					RHICmdList,
					*ExistResourceRequest,
					RHITextureToSend,
					SendIt.Value.ColorDesc,
					GPUIndex,
					SendIt.Value.GetRectIfDefined()
					);
			}
		}
	}

	// and receive custom textures
	for (const TPair<FString, FTextureShareWorldSubsystemRenderTargetResourceProxy>& ReceiveIt : Receive)
	{
		// Share only requested resources
		if (const FTextureShareCoreResourceRequest* ExistResourceRequest = ObjectProxy.GetData_RenderThread().FindResourceRequest(
			FTextureShareCoreResourceDesc(
				ReceiveIt.Key.ToLower(),
				ETextureShareTextureOp::Write,
				ETextureShareSyncStep::FrameProxyPreRenderEnd)))
		{
			if (FTextureRenderTargetResource* const DestRTT = ReceiveIt.Value.RenderTarget)
			{
				// Send if remote process request to read this texture
				ObjectProxy.ShareRenderTargetResource_RenderThread(
					RHICmdList,
					*ExistResourceRequest,
					DestRTT,
					ReceiveIt.Value.ColorDesc,
					GPUIndex,
					ReceiveIt.Value.GetRectIfDefined());
			}
		}
	}

	ObjectProxy.FrameSync_RenderThread(RHICmdList, ETextureShareSyncStep::FrameProxyPreRenderEnd);

	// End frame when no back buffer is used
	if (!ShouldUseBackbufferTexture(ObjectProxy))
	{
		ObjectProxy.EndFrameSync_RenderThread(RHICmdList);
	}
}

void FTextureShareWorldSubsystemContext::OnTextureSharePreRenderViewFamily_RenderThread(FRHICommandListImmediate& RHICmdList, const ITextureShareObjectProxy& ObjectProxy)
{
	// Ignore other contexts and render thread logic if the game thread fails.
	if (!bGameThreadSynchronized || ObjectProxy.GetTextureShareContext_RenderThread() != this)
	{
		return;
	}

	UE_TS_LOG(LogTextureShareWorldSubsystem, Log, TEXT("%s:OnTextureSharePreRenderViewFamily_RenderThread()"), *ObjectProxy.GetName_RenderThread());

	bRenderThreadFrameStarted = true;

	ObjectProxy.BeginFrameSync_RenderThread(RHICmdList);
}

void FTextureShareWorldSubsystemContext::OnTextureSharePostRenderViewFamily_RenderThread(FRHICommandListImmediate& RHICmdList, const ITextureShareObjectProxy& ObjectProxy)
{
	// Ignore other contexts and render thread logic if the game thread fails.
	if (!bGameThreadSynchronized || ObjectProxy.GetTextureShareContext_RenderThread() != this)
	{
		return;
	}

	UE_TS_LOG(LogTextureShareWorldSubsystem, Log, TEXT("%s:OnTextureSharePostRenderViewFamily_RenderThread()"), *ObjectProxy.GetName_RenderThread());

	ShareResources_RenderThread(RHICmdList, ObjectProxy);
}

void FTextureShareWorldSubsystemContext::GameViewportEndDraw_RenderThread(FRHICommandListImmediate& RHICmdList, const ITextureShareObjectProxy& ObjectProxy)
{
	// If these calls have already been called from the VE, we skip this logic:
	// OnTextureSharePreRenderViewFamily_RenderThread(), OnTextureSharePostRenderViewFamily_RenderThread()
	if (bRenderThreadFrameStarted)
	{
		return;
	}

	// Ignore other contexts and render thread logic if the game thread fails.
	if (!bGameThreadSynchronized || ObjectProxy.GetTextureShareContext_RenderThread() != this)
	{
		return;
	}

	UE_TS_LOG(LogTextureShareWorldSubsystem, Log, TEXT("%s:GameViewportEndDraw_RenderThread()"), *ObjectProxy.GetName_RenderThread());

	// If the VE callbacks have not been called before, we must implement different logic here
	bRenderThreadFrameStarted = true;
	ObjectProxy.BeginFrameSync_RenderThread(RHICmdList);
	ShareResources_RenderThread(RHICmdList, ObjectProxy);
}

void FTextureShareWorldSubsystemContext::OnTextureShareBackBufferReadyToPresent_RenderThread(SWindow& SlateWindow, const FTextureRHIRef& InBackbuffer, const ITextureShareObjectProxy& ObjectProxy)
{
	// Ignore other contexts and render thread logic if the game thread fails.
	if (!bGameThreadSynchronized || ObjectProxy.GetTextureShareContext_RenderThread() != this)
	{
		return;
	}

	if (!ShouldUseBackbufferTexture(ObjectProxy))
	{
		return;
	}

	using namespace UE::TextureShareWorldSubsystemContext;
	UE_TS_LOG(LogTextureShareWorldSubsystem, Log, TEXT("%s:OnTextureShareBackBufferReadyToPresent_RenderThread()"), *ObjectProxy.GetName_RenderThread());

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	if (ObjectProxy.IsFrameSyncActive_RenderThread())
	{
		// Share backbuffer and sync
		FRHITexture* RHIBackbufferTexture = InBackbuffer.IsValid() ? InBackbuffer.GetReference() : nullptr;
		if (RHIBackbufferTexture
			&& ObjectProxy.IsFrameSyncActive_RenderThread())
		{
			// mgpu is currently not handled by this logic
			const int32 GPUIndex = -1;

			const float BackbufferGamma = GetBackbufferGamma();

			// Gathering UE texture color information
			const FTextureShareColorDesc BackbufferColorDesc(BackbufferGamma);

			// Send if remote process request to read this texture
			ObjectProxy.ShareResource_RenderThread(RHICmdList,
				FTextureShareCoreResourceDesc(
					UE::TextureShareStrings::SceneTextures::Backbuffer,
					ETextureShareTextureOp::Read),
				RHIBackbufferTexture, BackbufferColorDesc, GPUIndex);

			// Receive if remote process request to write this texture
			ObjectProxy.ShareResource_RenderThread(RHICmdList,
				FTextureShareCoreResourceDesc(
					UE::TextureShareStrings::SceneTextures::Backbuffer,
					ETextureShareTextureOp::Write,
					ETextureShareSyncStep::FrameProxyBackBufferReadyToPresentEnd),
				RHIBackbufferTexture, BackbufferColorDesc, GPUIndex);
		}
	}

	ObjectProxy.FrameSync_RenderThread(RHICmdList, ETextureShareSyncStep::FrameProxyBackBufferReadyToPresentEnd);

	// End frame
	ObjectProxy.EndFrameSync_RenderThread(RHICmdList);
}

void FTextureShareWorldSubsystemContext::OnTextureShareGameViewportBeginDraw(ITextureShareObject& Object)
{
	// Ignore other contexts
	if (Object.GetTextureShareContext() != this)
	{
		return;
	}

	UE_TS_LOG(LogTextureShareWorldSubsystem, Log, TEXT("%s:OnTextureShareGameViewportBeginDraw()"), *Object.GetName());

	if (!bGameThreadUpdated)
	{
		// Call once
		bGameThreadUpdated = true;

		// Game thread logic should be called once per frame
		Tick_GameThread(Object);
	}
}

void FTextureShareWorldSubsystemContext::OnTextureShareGameViewportEndDraw(ITextureShareObject& Object)
{
	if (!bGameThreadSynchronized)
	{
		// Ignore if game thread sync failed.
		return;
	}

	// Ignore other contexts
	if (Object.GetTextureShareContext() != this)
	{
		return;
	}

	UE_TS_LOG(LogTextureShareWorldSubsystem, Log, TEXT("%s:OnTextureShareGameViewportEndDraw()"), *Object.GetName());

	ENQUEUE_RENDER_COMMAND(TextureShareWorldSubsystemContext_EndDraw)(
		[
			Context = SharedThis(this),
			ObjectProxyRef = Object.GetProxy()
		](FRHICommandListImmediate& RHICmdList)
	{
		Context->GameViewportEndDraw_RenderThread(RHICmdList, *ObjectProxyRef);
	});
}

void FTextureShareWorldSubsystemContext::OnTextureShareBeginRenderViewFamily(FSceneViewFamily& ViewFamily, ITextureShareObject& Object)
{
	// Ignore other contexts
	if (Object.GetTextureShareContext() != this)
	{
		return;
	}

	// This implementation supports only specific viewfamily
	using namespace UE::TextureShareWorldSubsystemContext;
	if (!IsViewFamilySupported(ViewFamily))
	{
		return;
	}

	UE_TS_LOG(LogTextureShareWorldSubsystem, Log, TEXT("%s:OnTextureShareBeginRenderViewFamily()"), *Object.GetName());

	// Game thread logic should be called once per frame
	if (!bGameThreadUpdated)
	{
		// Call once
		bGameThreadUpdated = true;

		// Game thread logic should be called once per frame
		Tick_GameThread(Object);
	}
}

void FTextureShareWorldSubsystemContext::RegisterTextureShareContextCallbacks()
{
	using namespace UE::TextureShareWorldSubsystemContext;

	check(IsInGameThread());

	if (!bGameThreadCallbacksRegistered)
	{
		bGameThreadCallbacksRegistered = true;

		// Register callbacks for the game thread.
		GetTextureShareAPI().GetCallbacks().OnTextureShareBeginRenderViewFamily().AddRaw(this, &FTextureShareWorldSubsystemContext::OnTextureShareBeginRenderViewFamily);
		GetTextureShareAPI().GetCallbacks().OnTextureShareGameViewportBeginDraw().AddRaw(this, &FTextureShareWorldSubsystemContext::OnTextureShareGameViewportBeginDraw);
		GetTextureShareAPI().GetCallbacks().OnTextureShareGameViewportEndDraw().AddRaw(this, &FTextureShareWorldSubsystemContext::OnTextureShareGameViewportEndDraw);
	}
}

void FTextureShareWorldSubsystemContext::UnregisterTextureShareContextCallbacks()
{
	using namespace UE::TextureShareWorldSubsystemContext;

	check(IsInRenderingThread());

	if (bGameThreadCallbacksRegistered)
	{
		bGameThreadCallbacksRegistered = false;

		// Unregister callbacks for the rendering thread.
		GetTextureShareAPI().GetCallbacks().OnTextureShareBeginRenderViewFamily().RemoveAll(this);
		GetTextureShareAPI().GetCallbacks().OnTextureShareGameViewportBeginDraw().RemoveAll(this);
		GetTextureShareAPI().GetCallbacks().OnTextureShareGameViewportEndDraw().RemoveAll(this);
	}
}

void FTextureShareWorldSubsystemContext::RegisterTextureShareContextCallbacks_RenderThread()
{
	using namespace UE::TextureShareWorldSubsystemContext;

	check(IsInRenderingThread());

	if (!bRTCallbacksRegistered)
	{
		bRTCallbacksRegistered = true;

		// Register callbacks for the rendering thread.
		GetTextureShareAPI().GetCallbacks().OnTextureSharePreRenderViewFamily_RenderThread().AddRaw(this, &FTextureShareWorldSubsystemContext::OnTextureSharePreRenderViewFamily_RenderThread);
		GetTextureShareAPI().GetCallbacks().OnTextureSharePostRenderViewFamily_RenderThread().AddRaw(this, &FTextureShareWorldSubsystemContext::OnTextureSharePostRenderViewFamily_RenderThread);
		GetTextureShareAPI().GetCallbacks().OnTextureShareBackBufferReadyToPresent_RenderThread().AddRaw(this, &FTextureShareWorldSubsystemContext::OnTextureShareBackBufferReadyToPresent_RenderThread);
	}
}

/** Unregister callbacks for the rendering thread. */
void FTextureShareWorldSubsystemContext::UnregisterTextureShareContextCallbacks_RenderThread()
{
	using namespace UE::TextureShareWorldSubsystemContext;

	check(IsInRenderingThread());

	if (bRTCallbacksRegistered)
	{
		bRTCallbacksRegistered = false;

		// Unregister callbacks for the rendering thread.
		GetTextureShareAPI().GetCallbacks().OnTextureSharePreRenderViewFamily_RenderThread().RemoveAll(this);
		GetTextureShareAPI().GetCallbacks().OnTextureSharePostRenderViewFamily_RenderThread().RemoveAll(this);
		GetTextureShareAPI().GetCallbacks().OnTextureShareBackBufferReadyToPresent_RenderThread().RemoveAll(this);
	}
}