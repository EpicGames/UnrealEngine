// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ITextureShareWorldSubsystemContext.h"
#include "TextureResource.h"
#include "Containers/TextureShareContainers_Color.h"
#include "Game/WorldSubsystem/TextureShareWorldSubsystemProxy.h"

class ITextureShareObject;
class ITextureShareObjectProxy;
class SWindow;
class FSceneViewFamily;
class UTextureShareObject;

/**
* Custom implementation of TextureShare context
* All ITextureShareCallbacks must be implemented here.
*/
struct FTextureShareWorldSubsystemContext
	: public ITextureShareWorldSubsystemContext
{
	// ~Begin ITextureShareContext
	virtual ~FTextureShareWorldSubsystemContext() = default;

	virtual void RegisterTextureShareContextCallbacks() override;
	virtual void UnregisterTextureShareContextCallbacks() override;

	virtual void RegisterTextureShareContextCallbacks_RenderThread() override;
	virtual void UnregisterTextureShareContextCallbacks_RenderThread() override;

	// ~~ End ITextureShareContext

	/** Implement tick for the TextureShare object.
	* @param TextureShareObject  - (in) TS object to process
	* @param TextureShareUObject - (in,out) TS UObject.
	* @param Viewport            - the viewport that shared by TS
	*/
	void Tick(
		ITextureShareObject& Object,
		UTextureShareObject& TextureShareUObject,
		FViewport* InViewport);

	/** Called on rendering thread, after viewfamily is rendered.*/
	void GameViewportEndDraw_RenderThread(FRHICommandListImmediate& RHICmdList, const ITextureShareObjectProxy& ObjectProxy);

protected:
	/////////////// Callbacks used by this implementation:

	/** Called for each view family that are rendered in the frame. */
	void OnTextureShareBeginRenderViewFamily(FSceneViewFamily& ViewFamily, ITextureShareObject& Object);

	/** Called from the FTextureShareSceneViewExtension in the rendering thread just before rendering starts. */
	void OnTextureSharePreRenderViewFamily_RenderThread(FRHICommandListImmediate& RHICmdList, const ITextureShareObjectProxy& ObjectProxy);

	/** Called from the FTextureShareSceneViewExtension in the rendering thread immediately after rendering completes. */
	void OnTextureSharePostRenderViewFamily_RenderThread(FRHICommandListImmediate& RHICmdList, const ITextureShareObjectProxy& ObjectProxy);

	/** Called from the FTextureShareSceneViewExtension in the rendering thread just before present. */
	void OnTextureShareBackBufferReadyToPresent_RenderThread(SWindow& SlateWindow, const FTextureRHIRef& RHITexture, const ITextureShareObjectProxy& ObjectProxy);

	/** GameViewport event onBeginDraw. */
	void OnTextureShareGameViewportBeginDraw(ITextureShareObject& Object);

	/** GameViewport event onEndDraw. */
	void OnTextureShareGameViewportEndDraw(ITextureShareObject& Object);

private:

	/** Update TS object on the game thread. */
	void Tick_GameThread(ITextureShareObject& TextureShareObject);

	/** Returns true is backbuffer texture is shared. */
	bool ShouldUseBackbufferTexture(const ITextureShareObjectProxy& ObjectProxy) const;

	/**
	* Share resources of this context
	*/
	void ShareResources_RenderThread(FRHICommandListImmediate& RHICmdList, const ITextureShareObjectProxy& ObjectProxy) const;

public:
	// Named resources to send
	TMap<FString, FTextureShareWorldSubsystemTextureProxy> Send;

	// Named Resources to receive
	TMap<FString, FTextureShareWorldSubsystemRenderTargetResourceProxy> Receive;

	TObjectPtr<UTextureShareObject> TextureShareUObject;

	// Is rendering thread callbacks are registered.
	bool bRTCallbacksRegistered = false;

	// Is Game thread callbacks are registered.
	bool bGameThreadCallbacksRegistered = false;

	// The game thread logic has already been updated. This flag is true even if synchronization in the game thread fails.
	bool bGameThreadUpdated = false;

	// Is the game thread synchronization a success
	bool bGameThreadSynchronized = false;

	// Is frame started on rendering thread
	bool bRenderThreadFrameStarted = false;
};
