// Copyright Epic Games, Inc. All Rights Reserved.

#include "Game/WorldSubsystem/TextureShareWorldSubsystemProxy.h"
#include "Blueprints/TextureShareBlueprintContainers.h"
#include "TextureResource.h"

bool FTextureShareWorldSubsystemProxyBase::IsRectValid() const
{
	if (const FIntRect* DefinedRectPtr = GetRectIfDefined())
	{
		if (DefinedRectPtr->Size().GetMin() <= 0)
		{
			return false;
		}
	}

	return true;
}

const FIntRect* FTextureShareWorldSubsystemProxyBase::GetRectIfDefined() const
{
	if (!Rect.IsEmpty())
	{
		return &Rect;
	}

	return nullptr;
}

FTextureShareWorldSubsystemTextureProxy::FTextureShareWorldSubsystemTextureProxy(const FTextureShareSendTextureDesc& InSendTextureDesc)
{
	if (UTexture* SrcTexture = InSendTextureDesc.Texture)
	{
		// Get resource
		Texture = SrcTexture->GetResource();

		// Get Color info (gamma, etc)
		{
			// UTextures always uses a linear gamut
			const float TexturesGamma = 1.0f;

			// Gathering UE texture color information
			const FTextureShareColorDesc TextureColorDesc(TexturesGamma);

			ColorDesc = TextureColorDesc;
		}
	}
}


bool FTextureShareWorldSubsystemTextureProxy::IsEnabled() const
{
	return Texture && IsRectValid();
}

FTextureShareWorldSubsystemRenderTargetResourceProxy::FTextureShareWorldSubsystemRenderTargetResourceProxy(const FTextureShareReceiveTextureDesc& InReceiveTextureDesc)
{
	if (FTextureRenderTargetResource* ReceiveTexture = InReceiveTextureDesc.Texture ? InReceiveTextureDesc.Texture->GameThread_GetRenderTargetResource() : nullptr)
	{
		RenderTarget = ReceiveTexture;

		// Get Color info (gamma, etc)
		{
			// A linear gamut is always used for RTT
			const float TexturesGamma = 1.0f;

			// Gathering UE texture color information
			const FTextureShareColorDesc TextureColorDesc(TexturesGamma);

			ColorDesc = TextureColorDesc;
		}
	}
}


bool FTextureShareWorldSubsystemRenderTargetResourceProxy::IsEnabled() const
{
	return RenderTarget && IsRectValid();
}