// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Blueprints/TextureShareBlueprintContainers.h"
#include "Containers/TextureShareContainers_Color.h"

struct FTextureShareSendTextureDesc;
struct FTextureShareReceiveTextureDesc;
class FTextureResource;
class FTextureRenderTargetResource;

/** Proxy data base class. */
struct FTextureShareWorldSubsystemProxyBase
{
	/** Is rect valid and can be used. */
	bool IsRectValid() const;

	/** Returns a pointer to the region if defined, or nullptr. */
	const FIntRect* GetRectIfDefined() const;

	// Color desc (gamma, etc)
	FTextureShareColorDesc ColorDesc;

	// Region on texture
	FIntRect Rect = FIntRect(FIntPoint::ZeroValue, FIntPoint::ZeroValue);
};

/** Proxy data for textures. */
struct FTextureShareWorldSubsystemTextureProxy
	: public FTextureShareWorldSubsystemProxyBase
{
	FTextureShareWorldSubsystemTextureProxy(const FTextureShareSendTextureDesc& InSendTextureDesc);

	/** Is this proxy can be used. */
	bool IsEnabled() const;

	// Texture resource
	FTextureResource* Texture = nullptr;
};

/** Proxy data for RTT. */
struct FTextureShareWorldSubsystemRenderTargetResourceProxy
	: public FTextureShareWorldSubsystemProxyBase
{
	FTextureShareWorldSubsystemRenderTargetResourceProxy(const FTextureShareReceiveTextureDesc& InReceiveTextureDesc);

	/** Is this proxy can be used. */
	bool IsEnabled() const;

	// RTT resource
	FTextureRenderTargetResource* RenderTarget = nullptr;
};
