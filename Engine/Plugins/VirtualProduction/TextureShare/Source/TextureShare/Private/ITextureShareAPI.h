// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

#include "Containers/TextureShareCoreEnums.h"

class ITextureShareObject;
class ITextureShareObjectProxy;
class UWorld;
class ITextureShareCallbacks;
class FSceneViewExtensionBase;

/**
 * TextureShare API
 */
class TEXTURESHARE_API ITextureShareAPI
{
public:
	virtual ~ITextureShareAPI() = default;

public:
	/**
	 * Handle BeginPlay event
	 *
	 * @param InWorld - world
	 *
	 * @return - none
	 */
	virtual void OnWorldBeginPlay(UWorld& InWorld) = 0;

	/**
	 * Handle EndPlay event
	 *
	 * @param InWorld - world
	 *
	 * @return - none
	 */
	virtual void OnWorldEndPlay(UWorld& InWorld) = 0;

public:
	/**
	 * Get or create new TextureShare object
	 *
	 * @param ShareName - Unique share name (case insensitive)
	 *
	 * @return - Ptr to TextureShare object
	 */
	virtual TSharedPtr<ITextureShareObject, ESPMode::ThreadSafe> GetOrCreateObject(const FString& ShareName, const ETextureShareProcessType InProcessType = ETextureShareProcessType::Undefined) = 0;

	/**
	 * Remove TextureShare object
	 *
	 * @param ShareName - Unique share name (case insensitive)
	 *
	 * @return True if the success
	 */
	virtual bool RemoveObject(const FString& ShareName) = 0;

	/**
	 * The test for the TextureShare object with the specified name still exists.
	 *
	 * @param ShareName - Unique share name (case insensitive)
	 *
	 * @return - true, if object with the specified name still exists.
	 */
	virtual bool IsObjectExist(const FString& ShareName) const = 0;

public:
	/**
	* Get TextureShare object
	*
	* @param ShareName - Unique share name (case insensitive)
	*
	* @return - Ptr to TextureShare object
	*/
	virtual TSharedPtr<ITextureShareObject, ESPMode::ThreadSafe> GetObject(const FString& ShareName) const = 0;

	/**
	 * Get TextureShare proxy object
	 *
	 * @param ShareName - Unique share name (case insensitive)
	 *
	 * @return - Ptr to TextureShare proxy object
	 */
	virtual TSharedPtr<ITextureShareObjectProxy, ESPMode::ThreadSafe> GetObjectProxy_RenderThread(const FString& ShareName) const = 0;

	/**
	* Get TextureShare object
	*
	* @param InViewExtension - View extension that used by TextureShare object.
	*
	* @return - Ptr to TextureShare object
	*/
	virtual TSharedPtr<ITextureShareObject, ESPMode::ThreadSafe> GetObject(const TSharedRef<const FSceneViewExtensionBase, ESPMode::ThreadSafe>& InViewExtension) const = 0;

	/**
	 * Get TextureShare proxy object
	 *
	* @param InViewExtension - View extension that used by TextureShare proxy object.
	 *
	 * @return - Ptr to TextureShare proxy object
	 */
	virtual TSharedPtr<ITextureShareObjectProxy, ESPMode::ThreadSafe> GetObjectProxy_RenderThread(const TSharedRef<const FSceneViewExtensionBase, ESPMode::ThreadSafe>& InViewExtension) const = 0;

public:
	/**
	 * Returns a list of handles to interprocess TextureShare objects with the specified name.
	 *
	 * @param InShareName            - (optional) name of the TextureShare object
	 * @param OutInterprocessObjects - Output array with handles to TextureShare objects
	 *
	 * @return true, if success
	 */
	virtual bool GetInterprocessObjects(const FString& InShareName, TArray<struct FTextureShareCoreObjectDesc>& OutInterprocessObjects) const = 0;

	/**
	 * Returns detailed information about the local process
	 */
	virtual const struct FTextureShareCoreObjectProcessDesc& GetProcessDesc() const = 0;

	/**
	 * Assign a unique name to the local process. The changes only affect newly created TextureShare objects.
	 *
	 * @param InProcessId - Custom local process name
	 */
	virtual void SetProcessName(const FString& InProcessId) = 0;

public:
	/**
	* Access to the TextureShare callbacks API
	*
	* @return Callbacks API
	*/
	virtual ITextureShareCallbacks& GetCallbacks() = 0;

public:
	/**
	 * Disable the multi-object world subsystem. This disabled state will persist until the EnableWorldSubsystem() pair function is called on all objects.
	 *
	 * @param InObject - external object
	 *
	 * @return none
	 */
	UE_DEPRECATED(5.6, "This function has been deprecated. Use the ITextureShareContext class to separate different implementations.")
	virtual void DisableWorldSubsystem(const uint8* InObject) { };

	/**
	 * Enable WorldSubsystem.
	 *
	 * @param InObject - external object
	 *
	 * @return none
	 */
	UE_DEPRECATED(5.6, "This function has been deprecated. Use the ITextureShareContext class to separate different implementations.")
	virtual void EnableWorldSubsystem(const uint8* InObject) { };

	/**
	 * Return true if WorldSubsystem.can by used
	 */
	UE_DEPRECATED(5.6, "This function has been deprecated. Use the ITextureShareContext class to separate different implementations.")
	virtual bool IsWorldSubsystemEnabled() const { return false; };
};
