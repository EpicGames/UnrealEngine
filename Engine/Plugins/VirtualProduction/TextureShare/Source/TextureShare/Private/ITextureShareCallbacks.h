// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"

#include "Containers/TextureShareContainers.h"
#include "Containers/TextureShareCoreEnums.h"

class ITextureShareObject;
class ITextureShareObjectProxy;
class FRHICommandListImmediate;
class FSceneViewFamily;
class SWindow;

/**
 * TextureShare callbacks API:
 * 
 * The same callbacks can be used by multiple implementations, and all of them will be invoked by broadcast calls for each TS object.
 * This will cause a single TS object to be consistently called by all callbacks of the same type from all existing implementations.
 * To avoid this situation, we must filter the TS objects within each callback by implementation type.
 * 
 * Each custom implementation must create a new user context class based on the ITextureShareContext class.
 * Each implementation must then assign this new context to all TS objects it owns
 * using the ITextureShareObject::SetTextureShareContext() function.
 *
 * Within each callback, the context of the TS object must be checked for the implementation type.
 * 
 * The context of a TextureShare object can be retrieved using one of two functions:
 * ITextureShareObject::GetTextureShareContext() and
 * ITextureShareObject::GetTextureShareContext_RenderThread()
 * 
 * The context class can be checked using the ITextureShareContext::IsA() function. 
 * 
 * Game thread callbacks must be registered and unregistered using these two overridden functions:
 * ITextureShareContext::RegisterTextureShareContextCallbacks() and
 * ITextureShareContext::UnregisterTextureShareContextCallbacks().
 * 
 * Rendering thread callbacks must be registered and unregistered using these two overridden functions:
 * ITextureShareContext::RegisterTextureShareContextCallbacks_RenderThread() and
 * ITextureShareContext::UnregisterTextureShareContextCallbacks_RenderThread().
 */
class ITextureShareCallbacks
{
public:
	static ITextureShareCallbacks& Get();

	virtual ~ITextureShareCallbacks() = default;

public:
	/** 
	* This is a subscription to the UGameViewportClient::OnBeginDraw() callback.
	* Accessor for the delegate called when the engine starts drawing a game viewport
	*/
	DECLARE_EVENT_OneParam(ITextureShareCallbacks, FTextureShareGameViewportBeginDrawEvent, ITextureShareObject&);
	virtual FTextureShareGameViewportBeginDrawEvent& OnTextureShareGameViewportBeginDraw() = 0;

	/**
	* This is a subscription to the UGameViewportClient::OnDraw() callback.
	* Accessor for the delegate called when the game viewport is drawn, before drawing the console
	*/
	DECLARE_EVENT_OneParam(ITextureShareCallbacks, FTextureShareGameViewportDrawEvent, ITextureShareObject&);
	virtual FTextureShareGameViewportDrawEvent& OnTextureShareGameViewportDraw() = 0;

	/**
	* This is a subscription to the UGameViewportClient::OnEndDraw() callback.
	* Accessor for the delegate called when the engine finishes drawing a game viewport
	*/
	DECLARE_EVENT_OneParam(ITextureShareCallbacks, FTextureShareGameViewportEndDrawEvent, ITextureShareObject&);
	virtual FTextureShareGameViewportEndDrawEvent& OnTextureShareGameViewportEndDraw() = 0;

public:
	/** Called on begin frame sync **/
	DECLARE_EVENT_TwoParams(ITextureShareCallbacks, FTextureShareBeginRenderViewFamilyEvent, FSceneViewFamily&, ITextureShareObject&);
	virtual FTextureShareBeginRenderViewFamilyEvent& OnTextureShareBeginRenderViewFamily() = 0;

	/** Called on session start	**/
	DECLARE_EVENT_OneParam(ITextureShareCallbacks, FTextureShareBeginSessionEvent, ITextureShareObject&);
	virtual FTextureShareBeginSessionEvent& OnTextureShareBeginSession() = 0;

	/** Called on session end **/
	DECLARE_EVENT_OneParam(ITextureShareCallbacks, FTextureShareEndSessionEvent, ITextureShareObject&);
	virtual FTextureShareEndSessionEvent& OnTextureShareEndSession() = 0;

	/** Called on begin frame sync **/
	DECLARE_EVENT_OneParam(ITextureShareCallbacks, FTextureSharePreBeginFrameSyncEvent, ITextureShareObject&);
	virtual FTextureSharePreBeginFrameSyncEvent& OnTextureSharePreBeginFrameSync() = 0;

	/** Called on begin frame sync **/
	DECLARE_EVENT_OneParam(ITextureShareCallbacks, FTextureShareBeginFrameSyncEvent, ITextureShareObject&);
	virtual FTextureShareBeginFrameSyncEvent& OnTextureShareBeginFrameSync() = 0;

	/** Called on end frame sync **/
	DECLARE_EVENT_OneParam(ITextureShareCallbacks, FTextureShareEndFrameSyncEvent, ITextureShareObject&);
	virtual FTextureShareEndFrameSyncEvent& OnTextureShareEndFrameSync() = 0;

	/** Called on frame sync **/
	DECLARE_EVENT_TwoParams(ITextureShareCallbacks, FTextureShareFrameSyncEvent, ITextureShareObject&, const ETextureShareSyncStep);
	virtual FTextureShareFrameSyncEvent& OnTextureShareFrameSync() = 0;


	/**
	* This is a redirected event from the game thread, just before frame synchronization.
	* If game stream synchronization fails, the proxy context remains from the previous frame.
	* This callback is useful for preparing context and other data for the new frame.
	*/
	DECLARE_EVENT_TwoParams(ITextureShareCallbacks, FTextureSharePreBeginFrameSyncEvent_RenderThread, FRHICommandListImmediate&, const ITextureShareObjectProxy&);
	virtual FTextureSharePreBeginFrameSyncEvent_RenderThread& OnTextureSharePreBeginFrameSync_RenderThread() = 0;

	/** Called on begin frame sync on render thread **/
	DECLARE_EVENT_TwoParams(ITextureShareCallbacks, FTextureShareBeginFrameSyncEvent_RenderThread, FRHICommandListImmediate&, const ITextureShareObjectProxy&);
	virtual FTextureShareBeginFrameSyncEvent_RenderThread& OnTextureShareBeginFrameSync_RenderThread() = 0;

	/** Called on end frame sync on render thread **/
	DECLARE_EVENT_TwoParams(ITextureShareCallbacks, FTextureShareEndFrameSyncEvent_RenderThread, FRHICommandListImmediate&, const ITextureShareObjectProxy&);
	virtual FTextureShareEndFrameSyncEvent_RenderThread& OnTextureShareEndFrameSync_RenderThread() = 0;

	/** Called on frame sync on render thread **/
	DECLARE_EVENT_ThreeParams(ITextureShareCallbacks, FTextureShareFrameSyncEvent_RenderThread, FRHICommandListImmediate&, const ITextureShareObjectProxy&, const ETextureShareSyncStep);
	virtual FTextureShareFrameSyncEvent_RenderThread& OnTextureShareFrameSync_RenderThread() = 0;

	/** Called from the FTextureShareSceneViewExtension in the rendering thread just before rendering starts. */
	DECLARE_EVENT_TwoParams(ITextureShareCallbacks, FTextureSharePreRenderViewFamily_RenderThread, FRHICommandListImmediate&, const ITextureShareObjectProxy&);
	virtual FTextureSharePreRenderViewFamily_RenderThread& OnTextureSharePreRenderViewFamily_RenderThread() = 0;

	/** Called from the FTextureShareSceneViewExtension in the rendering thread immediately after rendering completes. */
	DECLARE_EVENT_TwoParams(ITextureShareCallbacks, FTextureSharePostRenderViewFamily_RenderThread, FRHICommandListImmediate&, const ITextureShareObjectProxy&);
	virtual FTextureSharePostRenderViewFamily_RenderThread& OnTextureSharePostRenderViewFamily_RenderThread() = 0;

	/** Called from the FTextureShareSceneViewExtension in the rendering thread just before present. */
	DECLARE_EVENT_ThreeParams(ITextureShareCallbacks, FTextureShareBackBufferReadyToPresentEvent_RenderThread, SWindow&, const FTextureRHIRef&, const ITextureShareObjectProxy&);
	virtual FTextureShareBackBufferReadyToPresentEvent_RenderThread& OnTextureShareBackBufferReadyToPresent_RenderThread() = 0;
};
