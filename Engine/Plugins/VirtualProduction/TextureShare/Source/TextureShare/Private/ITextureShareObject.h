// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "ITextureShareContext.h"

#include "Containers/TextureShareContainers.h"
#include "Containers/TextureShareCoreContainers.h"

/**
 * TextureShare object interface (GameThread)
 */
class TEXTURESHARE_API ITextureShareObject
{
public:
	virtual ~ITextureShareObject() = default;

public:
	/** Assign new context data for this TS object on game thread
	* 
	* Every frame in the game thread we have to create a new context and populate it with new data.
	* Then set it using the ITextureShareObject::SetTextureShareContext() function. 
	* 
	* This context is passed from the game thread to the rendering thread.
	* (from the ITextureShareObject to the ITextureShareObjectProxy)
	* With this approach, we get unique context data for both threads.
	*/
	virtual void SetTextureShareContext(const TSharedPtr<ITextureShareContext, ESPMode::ThreadSafe>& InTextureShareContext) = 0;

	/** Returns a pointer to the assigned context. */
	virtual const ITextureShareContext* GetTextureShareContext() const = 0;

public:
	/////////////////////////// State ///////////////////////////

	/**
	* Return the name of the TextureShare.
	*/
	virtual const FString& GetName() const = 0;

	/**
	* Returns detailed information about the TextureShare object.
	*/
	virtual FTextureShareCoreObjectDesc GetObjectDesc() const = 0;

	/**
	 * Returns true if the TextureShare object is ready to be used.
	 */
	virtual bool IsActive() const = 0;

	/**
	 * Returns true if the TextureShareCore object has started a session and processes are connected for this frame.
	 */
	virtual bool IsFrameSyncActive() const = 0;

public:
	/////////////////////////// Settings ///////////////////////////

	/**
	 * Change the process name for this TextureShare object.
	 * If the object is synced now, then the changes are deferred and executed in the next frame.
	 *
	 * @param InProcessId - New process name
	 *
	 * @return true, if success
	 */
	virtual bool SetProcessId(const FString& InProcessId) = 0;

	/**
	 * Change the sync settings for this TextureShare object.
	 * If the object is synced now, then the changes are deferred and executed in the next frame.
	 *
	 * @param InSyncSetting - New sync settings
	 *
	 * @return true, if success
	 */
	virtual bool SetSyncSetting(const FTextureShareCoreSyncSettings& InSyncSetting) = 0;

	/**
	* Return the sync settings of the TextureShare.
	*/
	virtual FTextureShareCoreSyncSettings GetSyncSetting() const = 0;

	/**
	 * Get TextureShare default sync settings by template type
	 * The settings of this template are not related to the current settings of the object
	 *
	 * @param InType - Sync settings template type
	 *
	 * @return structure with sync logic settings
	 */
	virtual FTextureShareCoreFrameSyncSettings GetFrameSyncSettings(const ETextureShareFrameSyncTemplate InType) const = 0;

public:
	/////////////////////////// Session ///////////////////////////

	/**
	 * Start a TextureShare session for the specified viewport
	 *
	 * @param InViewport - (opt) ptr on rendering viewport
	 *
	 * @return True if the success
	 */
	virtual bool BeginSession(class FViewport* InViewport = nullptr) = 0;

	/**
	 * End the TextureShare session and the resources in use are also removed.
	 *
	 * @return True if the success
	 */
	virtual bool EndSession() = 0;

	/**
	 * Returns true if the session is currently valid
	 */
	virtual bool IsSessionActive() const = 0;


public:
	/////////////////////////// Interprocess Synchronization ///////////////////////////

	/**
	 * Begin sync logic in range FrameBegin..FrameEnd
	 * The list of connected processes for the current frame will also be updated.
	 * Returns true if the frame is connected to other processes that match the synchronization settings.
	 * Game and render thread are in sync
	 */
	virtual bool BeginFrameSync() = 0;

	/**
	 * Synchronize connected processes that support this sync step
	 * Data from remote processes will be read at the time the barrier is synchronized.
	 * Missed sync steps from the sync settings will also be called.
	 *
	 * @param InSyncStep - Sync step value
	 *
	 * @return True if the success
	 */
	virtual bool FrameSync(const ETextureShareSyncStep InSyncStep) = 0;

	/**
	 * Finalize sync logic in range FrameBegin..FrameEnd
	 * Missed sync steps from the sync settings will also be called.
	 * Game and render thread are in sync
	 * 
	 */
	virtual bool EndFrameSync() = 0;

	/**
	 * Returns a list of handles to the interprocess TextureShare objects that are currently connected to this object.
	 *
	 * @param OutInterprocessObjects - Output array with handles to TextureShare objects
	 *
	 * @return true, if success
	 */
	virtual TArray<FTextureShareCoreObjectDesc> GetConnectedInterprocessObjects() const = 0;

public:
	/////////////////////////// Data Containers ///////////////////////////

	/**
	 * Reference to Data.
	 * Object data for the current frame in the GameThread.
	 */
	virtual FTextureShareCoreData& GetCoreData() = 0;
	virtual const FTextureShareCoreData& GetCoreData() const = 0;

	/**
	 * Received Data from connected process objects
	 */
	virtual TArray<FTextureShareCoreObjectData> GetReceivedCoreObjectData() const = 0;

	/**
	 * Reference to views data
	 */
	virtual FTextureShareData& GetData() = 0;

	/**
	 * Reference to to view extension
	 */
	virtual TSharedPtr<class FTextureShareSceneViewExtension, ESPMode::ThreadSafe> GetViewExtension() const = 0;

public:
	/**
	 * Return proxy interface
	 */
	virtual TSharedRef<class ITextureShareObjectProxy, ESPMode::ThreadSafe> GetProxy() const = 0;
};
