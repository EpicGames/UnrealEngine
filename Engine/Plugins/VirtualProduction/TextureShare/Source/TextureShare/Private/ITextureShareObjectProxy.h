// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "ITextureShareContext.h"

#include "Containers/TextureShareContainers.h"
#include "Containers/TextureShareCoreContainers.h"
#include "Containers/TextureShareEnums.h"

class FTextureShareSceneViewExtension;
class FTextureRenderTargetResource;

/**
 * TextureShare object proxy interface (RenderingThread)
 */
class TEXTURESHARE_API ITextureShareObjectProxy
{
public:
	virtual ~ITextureShareObjectProxy() = default;

public:
	/** Returns a pointer to the assigned context on rendering thread. */
	virtual const ITextureShareContext* GetTextureShareContext_RenderThread() const = 0;

public:
	/////////////////////////// State ///////////////////////////

	/**
	 * Return the name of the TextureShare.
	 */
	virtual const FString& GetName_RenderThread() const = 0;

	/**
	* Returns detailed information about the TextureShare object.
	*/
	virtual FTextureShareCoreObjectDesc GetObjectDesc_RenderThread() const = 0;

	/**
	 * Returns true if the TextureShare object is ready to be used.
	 */
	virtual bool IsActive_RenderThread() const = 0;

	/**
	 * Returns true if the TextureShare object has started a session and processes are connected for this frame.
	 */
	virtual bool IsFrameSyncActive_RenderThread() const = 0;

public:
	/////////////////////////// Interprocess Synchronization ///////////////////////////

	/**
	 * Begin sync logic in range FrameProxyBegin..FrameProxyEnd
	 * Game and render thread are in sync
	 */
	virtual bool BeginFrameSync_RenderThread(FRHICommandListImmediate& RHICmdList) const = 0;

	/**
	 * Synchronize connected processes that support this sync step
	 * ProxyData from remote processes will be read at the time the barrier is synchronized.
	 * Missed sync steps from the sync settings will also be called.
	 *
	 * @param InSyncStep - Sync step value
	 *
	 * @return True if the success
	 */
	virtual bool FrameSync_RenderThread(FRHICommandListImmediate& RHICmdList, const ETextureShareSyncStep InSyncStep) const = 0;

	/**
	 * Finalize sync logic in range FrameProxyBegin..FrameProxyEnd
	 * Missed sync steps from the sync settings will also be called.
	 * Game and render thread are in sync
	 */
	virtual bool EndFrameSync_RenderThread(FRHICommandListImmediate& RHICmdList) const = 0;

public:
	/////////////////////////// Resource ///////////////////////////

	/**
	 * Share texture resource with ResourceRequest
	 *
	 * @param RHICmdList        - RHI cmd list
	 * @param InResourceDesc    - resource information
	 * @param InTexture         - Texture RHI resource
	 * @param InTextureColor    - InTexture color desc
	 * @param InTextureGPUIndex - Texture GPU index
	 * @param InTextureRect     - (optional) Share only region from InTexture
	 *
	 * @return True if the success
	 */
	virtual bool ShareResource_RenderThread(
		FRHICommandListImmediate& RHICmdList,
		const FTextureShareCoreResourceDesc& InResourceDesc,
		FRHITexture* InTexture,
		const FTextureShareColorDesc& InTextureColorDesc,
		const int32 InTextureGPUIndex,
		const FIntRect* InTextureRect = nullptr) const = 0;

	/**
	 * Share texture resource with ResourceRequest
	 *
	 * @param GraphBuilder      - RDG builder
	 * @param InResourceDesc    - resource information
	 * @param InTextureRef      - RDG Texture ref
	 * @param InTextureColor    - InTextureRef color desc
	 * @param InTextureGPUIndex - Texture GPU index
	 * @param InTextureRect     - (optional) Share only region from InTexture
	 *
	 * @return True if the success
	 */
	virtual bool ShareResource_RenderThread(
		FRDGBuilder& GraphBuilder,
		const FTextureShareCoreResourceDesc& InResourceDesc,
		const FRDGTextureRef& InTextureRef,
		const FTextureShareColorDesc& InTextureColorDesc,
		const int32 InTextureGPUIndex,
		const FIntRect* InTextureRect = nullptr) const = 0;

	/**
	 * Share texture resource with ResourceRequest
	 *
	 * @param RHICmdList        - RHI cmd list
	 * @param InResourceRequest - resource information
	 * @param InTexture         - Texture RHI resource
	 * @param InTextureColor    - InTexture color desc
	 * @param InTextureGPUIndex - Texture GPU index
	 * @param InTextureRect     - (optional) Share only region from InTexture
	 *
	 * @return True if the success
	 */
	virtual bool ShareResource_RenderThread(
		FRHICommandListImmediate& RHICmdList,
		const FTextureShareCoreResourceRequest& InResourceRequest,
		FRHITexture* InTexture,
		const FTextureShareColorDesc& InTextureColorDesc,
		const int32 InTextureGPUIndex,
		const FIntRect* InTextureRect = nullptr) const = 0;

	/**
	 * Share texture RTT with ResourceRequest
	 *
	 * @param RHICmdList             - RHI cmd list
	 * @param InResourceRequest      - resource information
	 * @param InRenderTargetResource - Texture Render Target RHI resource
	 * @param InRenderTargetColor    - Texture RTT color desc
	 * @param InRenderTargetGPUIndex - Texture RTT GPU index
	 * @param InRenderTargetRect     - (optional) Share only region from Texture RTT
	 *
	 * @return True if the success
	 */
	virtual bool ShareRenderTargetResource_RenderThread(
		FRHICommandListImmediate& RHICmdList,
		const FTextureShareCoreResourceRequest& InResourceRequest,
		FTextureRenderTargetResource* InRenderTargetResource,
		const FTextureShareColorDesc& InRenderTargetColorDesc,
		const int32 InRenderTargetGPUIndex,
		const FIntRect* InRenderTargetRect = nullptr) const = 0;

	/**
	 * Share texture resource with ResourceRequest
	 *
	 * @param GraphBuilder      - RDG builder
	 * @param InResourceRequest - resource information
	 * @param InTextureRef      - RDG Texture ref
	 * @param InTextureColor    - InTextureRef color desc
	 * @param InTextureGPUIndex - Texture GPU index
	 * @param InTextureRect     - (optional) Share only region from InTexture
	 *
	 * @return True if the success
	 */
	virtual bool ShareResource_RenderThread(
		FRDGBuilder& GraphBuilder,
		const FTextureShareCoreResourceRequest& InResourceRequest,
		const FRDGTextureRef& InTextureRef,
		const FTextureShareColorDesc& InTextureColorDesc,
		const int32 InTextureGPUIndex,
		const FIntRect* InTextureRect = nullptr) const = 0;

public:
	/////////////////////////// Data Containers ///////////////////////////

	/**
	 * Reference to ProxyData.
	 * Object proxy data for the current frame in the RenderThread.
	 */
	virtual FTextureShareCoreProxyData& GetCoreProxyData_RenderThread() = 0;
	virtual const FTextureShareCoreProxyData& GetCoreProxyData_RenderThread() const = 0;

	/**
	 * Received ProxyData from connected process objects
	 */
	virtual TArray<FTextureShareCoreObjectProxyData> GetReceivedCoreObjectProxyData_RenderThread() const = 0;

	/**
	*  Return proxy flags
	*/
	virtual ETextureShareObjectProxyFlags GetObjectProxyFlags() const = 0;

	/**
	*  Set new proxy flags
	*/
	virtual void SetObjectProxyFlags(const ETextureShareObjectProxyFlags InFlags) = 0;

	/**
	 * Get data from game thread
	 */
	virtual const FTextureShareData& GetData_RenderThread() const = 0;

	/**
	 * Get view extension ptr
	 */
	virtual const TSharedPtr<FTextureShareSceneViewExtension, ESPMode::ThreadSafe>& GetViewExtension_RenderThread() const = 0;

public:
	/**
	 * Share texture resource with ResourceRequest
	 *
	 * @param RHICmdList        - RHI cmd list
	 * @param InResourceDesc    - resource information
	 * @param InTexture         - Texture RHI resource
	 * @param InTextureGPUIndex - Texture GPU index
	 * @param InTextureRect     - (optional) Share only region from InTexture
	 *
	 * @return True if the success
	 */
	UE_DEPRECATED(5.6, "This function has been deprecated. Use the new function with other arguments. ")
	virtual bool ShareResource_RenderThread(FRHICommandListImmediate& RHICmdList, const FTextureShareCoreResourceDesc& InResourceDesc, FRHITexture* InTexture, const int32 InTextureGPUIndex, const FIntRect* InTextureRect = nullptr) const { return false; };

	/**
	 * Share texture resource with ResourceRequest
	 *
	 * @param GraphBuilder      - RDG builder
	 * @param InResourceDesc    - resource information
	 * @param InTextureRef      - RDG Texture ref
	 * @param InTextureGPUIndex - Texture GPU index
	 * @param InTextureRect     - (optional) Share only region from InTexture
	 *
	 * @return True if the success
	 */
	UE_DEPRECATED(5.6, "This function has been deprecated. Use the new function with other arguments. ")
	virtual bool ShareResource_RenderThread(FRDGBuilder& GraphBuilder, const FTextureShareCoreResourceDesc& InResourceDesc, const FRDGTextureRef& InTextureRef, const int32 InTextureGPUIndex, const FIntRect* InTextureRect = nullptr) const { return false; };

	/**
	 * Share texture resource with ResourceRequest
	 *
	 * @param RHICmdList        - RHI cmd list
	 * @param InResourceRequest - resource information
	 * @param InTexture         - Texture RHI resource
	 * @param InTextureGPUIndex - Texture GPU index
	 * @param InTextureRect     - (optional) Share only region from InTexture
	 *
	 * @return True if the success
	 */
	UE_DEPRECATED(5.6, "This function has been deprecated. Use the new function with other arguments. ")
	virtual bool ShareResource_RenderThread(FRHICommandListImmediate& RHICmdList, const FTextureShareCoreResourceRequest& InResourceRequest, FRHITexture* InTexture, const int32 InTextureGPUIndex, const FIntRect* InTextureRect = nullptr) const { return false; };

	/**
	 * Share texture resource with ResourceRequest
	 *
	 * @param GraphBuilder      - RDG builder
	 * @param InResourceRequest - resource information
	 * @param InTextureRef      - RDG Texture ref
	 * @param InTextureGPUIndex - Texture GPU index
	 * @param InTextureRect     - (optional) Share only region from InTexture
	 *
	 * @return True if the success
	 */
	UE_DEPRECATED(5.6, "This function has been deprecated. Use the new function with other arguments. ")
	virtual bool ShareResource_RenderThread(FRDGBuilder& GraphBuilder, const FTextureShareCoreResourceRequest& InResourceRequest, const FRDGTextureRef& InTextureRef, const int32 InTextureGPUIndex, const FIntRect* InTextureRect = nullptr) const { return false; };

};
