// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if __UNREAL__
#include "CoreMinimal.h"
#else
#include "windows.h"
#endif

/**
 * Default names for TextureShare module
 */
namespace UE::TextureShareStrings::SceneTextures
{
	// Read-only scene resources
	static constexpr auto SceneColor = TEXT("SceneColor");
	static constexpr auto SceneDepth = TEXT("SceneDepth");
	static constexpr auto SmallDepthZ = TEXT("SmallDepthZ");
	static constexpr auto GBufferA = TEXT("GBufferA");
	static constexpr auto GBufferB = TEXT("GBufferB");
	static constexpr auto GBufferC = TEXT("GBufferC");
	static constexpr auto GBufferD = TEXT("GBufferD");
	static constexpr auto GBufferE = TEXT("GBufferE");
	static constexpr auto GBufferF = TEXT("GBufferF");

	// Read-write RTTs
	static constexpr auto FinalColor = TEXT("FinalColor");
	static constexpr auto Backbuffer = TEXT("Backbuffer");
};
