// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/TextureShareAPI.h"
#include "Module/TextureShareLog.h"
#include "Object/TextureShareObject.h"

#include "Misc/TextureShareStrings.h"

#include "ITextureShareCore.h"
#include "ITextureShareCoreAPI.h"
#include "ITextureShareCoreObject.h"

#include "Framework/Application/SlateApplication.h"
#include "Slate/SlateViewportProvider.h"
#include "Engine/GameViewportClient.h"
#include "SceneTextures.h"

//////////////////////////////////////////////////////////////////////////////////////////////
namespace UE::TextureShare::APIHelpers
{
	static const FName RendererModuleName(TEXT("Renderer"));

	static ITextureShareCoreAPI& TextureShareCoreAPI()
	{
		static ITextureShareCoreAPI& TextureShareCoreAPISingleton = ITextureShareCore::Get().GetTextureShareCoreAPI();
		return TextureShareCoreAPISingleton;
	}

	static ETextureShareDeviceType GetTextureShareDeviceType()
	{
		switch (RHIGetInterfaceType())
		{
		case ERHIInterfaceType::D3D11:  return ETextureShareDeviceType::D3D11;
		case ERHIInterfaceType::D3D12:  return ETextureShareDeviceType::D3D12;
		case ERHIInterfaceType::Vulkan: return ETextureShareDeviceType::Vulkan;
		
		default:
			break;
		}

		return ETextureShareDeviceType::Undefined;
	}
};
using namespace UE::TextureShare::APIHelpers;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareAPI
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareAPI::FTextureShareAPI()
{
	UE_LOGF(LogTextureShare, Log, "TextureShare API has been instantiated");
}

FTextureShareAPI::~FTextureShareAPI()
{
	FScopeLock Lock(&ThreadDataCS);

	RemoveTextureShareObjectInstances();
	UnregisterRendererModuleCallbacks();

	UE_LOGF(LogTextureShare, Log, "TextureShare API has been destroyed");
}

//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<ITextureShareObject, ESPMode::ThreadSafe> FTextureShareAPI::GetOrCreateObject(const FString& ShareName, const ETextureShareProcessType InProcessType)
{
	check(IsInGameThread());

	FScopeLock Lock(&ThreadDataCS);

	const FString& ShareNameLwr = ShareName.ToLower();
	if (const TSharedPtr<FTextureShareObject, ESPMode::ThreadSafe>* ExistObjectPtr = Objects.Find(ShareNameLwr))
	{
		return *ExistObjectPtr;
	}

	//Create new
	TSharedPtr<ITextureShareCoreObject, ESPMode::ThreadSafe> CoreObject = TextureShareCoreAPI().GetOrCreateCoreObject(ShareName, InProcessType);
	if (CoreObject.IsValid())
	{
		// Set current DeviceType
		CoreObject->SetDeviceType(GetTextureShareDeviceType());

		// Create game thread object
		const TSharedPtr<FTextureShareObject, ESPMode::ThreadSafe> NewObject(new FTextureShareObject(CoreObject.ToSharedRef()));
		if (NewObject.IsValid())
		{
			// Register for game thread
			Objects.Emplace(ShareNameLwr, NewObject);

			//Register for render thread
			ENQUEUE_RENDER_COMMAND(TextureShare_CreateObjectProxy)(
				[TextureShareAPI = this, ShareNameLwr, NewObjectProxy = NewObject->GetObjectProxyRef()](FRHICommandListImmediate& RHICmdList)
			{
				TextureShareAPI->ObjectProxies.Emplace(ShareNameLwr, NewObjectProxy);
			});

			// Register UE callbacks to access to scene textures and final backbuffer
			RegisterRendererModuleCallbacks();

			UE_LOGF(LogTextureShare, Log, "Created new TextureShare object '%ls'", *ShareName);

			return NewObject;
		}

		// Failed
		CoreObject.Reset();
		TextureShareCoreAPI().RemoveCoreObject(ShareName);
	}

	UE_LOGF(LogTextureShare, Error, "CreateTextureShareObject '%ls' failed", *ShareName);

	return nullptr;
}

bool FTextureShareAPI::RemoveObject(const FString& ShareName)
{
	// May call from both threads
	FScopeLock Lock(&ThreadDataCS);

	const FString& ShareNameLwr = ShareName.ToLower();
	if (!Objects.Contains(ShareNameLwr))
	{
		UE_LOGF(LogTextureShare, Error, "Can't remove TextureShare '%ls' - not exist", *ShareName);
		return false;
	}

	Objects[ShareNameLwr].Reset();
	Objects.Remove(ShareNameLwr);

	ENQUEUE_RENDER_COMMAND(TextureShare_RemoveObjectProxy)(
		[TextureShareAPI = this, ShareNameLwr](FRHICommandListImmediate& RHICmdList)
	{
		TextureShareAPI->ObjectProxies.Remove(ShareNameLwr);
	});

	UE_LOGF(LogTextureShare, Log, "Removed TextureShare object '%ls'", *ShareName);

	return true;
}

bool FTextureShareAPI::IsObjectExist(const FString& ShareName) const
{
	check(IsInGameThread());

	FScopeLock Lock(&ThreadDataCS);

	return Objects.Contains(ShareName.ToLower());
}

//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<ITextureShareObject, ESPMode::ThreadSafe> FTextureShareAPI::GetObject(const FString& ShareName) const
{
	check(IsInGameThread());

	FScopeLock Lock(&ThreadDataCS);

	if (const TSharedPtr<FTextureShareObject, ESPMode::ThreadSafe>* ExistObject = Objects.Find(ShareName.ToLower()))
	{
		return *ExistObject;
	}

	return nullptr;
}

TSharedPtr<ITextureShareObjectProxy, ESPMode::ThreadSafe> FTextureShareAPI::GetObjectProxy_RenderThread(const FString& ShareName) const
{
	check(IsInRenderingThread());

	if (const TSharedPtr<FTextureShareObjectProxy, ESPMode::ThreadSafe>* ExistObject = ObjectProxies.Find(ShareName.ToLower()))
	{
		return *ExistObject;
	}

	return nullptr;
}

TSharedPtr<ITextureShareObject, ESPMode::ThreadSafe> FTextureShareAPI::GetObject(const TSharedRef<const FSceneViewExtensionBase, ESPMode::ThreadSafe>& InViewExtension) const
{
	check(IsInGameThread());

	FScopeLock Lock(&ThreadDataCS);

	for (const TPair<FString, TSharedPtr<FTextureShareObject, ESPMode::ThreadSafe>>& ObjectIt : Objects)
	{
		if (ObjectIt.Value.IsValid() && ObjectIt.Value->GetViewExtension() == InViewExtension)
		{
			return ObjectIt.Value;
		}
	}

	return nullptr;
}

TSharedPtr<ITextureShareObjectProxy, ESPMode::ThreadSafe> FTextureShareAPI::GetObjectProxy_RenderThread(const TSharedRef<const FSceneViewExtensionBase, ESPMode::ThreadSafe>& InViewExtension) const
{
	check(IsInRenderingThread());

	for (const TPair<FString, TSharedPtr<FTextureShareObjectProxy, ESPMode::ThreadSafe>>& ObjectProxyIt : ObjectProxies)
	{
		if (ObjectProxyIt.Value.IsValid() && ObjectProxyIt.Value->GetViewExtension_RenderThread() == InViewExtension)
		{
			return ObjectProxyIt.Value;
		}
	}

	return nullptr;
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareAPI::GetInterprocessObjects(const FString& InShareName, TArray<FTextureShareCoreObjectDesc>& OutInterprocessObjects) const
{
	TArraySerializable<FTextureShareCoreObjectDesc> InterprocessObjects;
	if (TextureShareCoreAPI().GetInterprocessObjects(InShareName, InterprocessObjects))
	{
		OutInterprocessObjects.Empty();
		OutInterprocessObjects.Append(InterprocessObjects);

		return true;
	}

	return false;
}

const FTextureShareCoreObjectProcessDesc& FTextureShareAPI::GetProcessDesc() const
{
	return TextureShareCoreAPI().GetProcessDesc();
}

void FTextureShareAPI::SetProcessName(const FString& InProcessId)
{
	TextureShareCoreAPI().SetProcessName(InProcessId);
}

//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareAPI::RemoveTextureShareObjectInstances()
{
	// May call from both threads
	FScopeLock Lock(&ThreadDataCS);

	// Remove all objects
	Objects.Empty();

	// Remove all proxy
	ENQUEUE_RENDER_COMMAND(TextureShare_RemoveAll)(
		[TextureShareAPI = this](FRHICommandListImmediate& RHICmdList)
	{
		TextureShareAPI->ObjectProxies.Empty();
	});
}

//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareAPI::OnResolvedSceneColor_RenderThread(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures)
{
	check(IsInRenderingThread());

	for (TPair<FString, TSharedPtr<FTextureShareObjectProxy, ESPMode::ThreadSafe>>& ProxyIt : ObjectProxies)
	{
		const TSharedPtr<FTextureShareSceneViewExtension, ESPMode::ThreadSafe> ViewExtensionPtr = ProxyIt.Value.IsValid() ? ProxyIt.Value->GetViewExtension_RenderThread() : nullptr;
		if (ViewExtensionPtr.IsValid())
		{
			ViewExtensionPtr->OnResolvedSceneColor_RenderThread(GraphBuilder, SceneTextures);
		}
	}
}

void FTextureShareAPI::OnBackBufferReadyToPresent_RenderThread(SWindow& Window, ISlateViewportProvider& ViewportProvider)
{
	check(IsInRenderingThread());

	for (TPair<FString, TSharedPtr<FTextureShareObjectProxy, ESPMode::ThreadSafe>>& ObjectProxyIt : ObjectProxies)
	{
		if (ObjectProxyIt.Value.IsValid())
		{
			// Perform callback logic
			if (Callbacks.OnTextureShareBackBufferReadyToPresent_RenderThread().IsBound())
			{
				Callbacks.OnTextureShareBackBufferReadyToPresent_RenderThread().Broadcast(Window, ViewportProvider.GetBackBufferResource(), *ObjectProxyIt.Value);
			}
		}
	}
}

void FTextureShareAPI::OnGameViewportBeginDraw()
{
	check(IsInGameThread());

	for (TPair<FString, TSharedPtr<FTextureShareObject, ESPMode::ThreadSafe>>& ObjectIt : Objects)
	{
		if (ObjectIt.Value.IsValid())
		{
			// Perform callback logic
			if (Callbacks.OnTextureShareGameViewportBeginDraw().IsBound())
			{
				Callbacks.OnTextureShareGameViewportBeginDraw().Broadcast(*ObjectIt.Value);
			}
		}
	}
}

void FTextureShareAPI::OnGameViewportDraw()
{
	check(IsInGameThread());

	for (TPair<FString, TSharedPtr<FTextureShareObject, ESPMode::ThreadSafe>>& ObjectIt : Objects)
	{
		if (ObjectIt.Value.IsValid())
		{
			// Perform callback logic
			if (Callbacks.OnTextureShareGameViewportDraw().IsBound())
			{
				Callbacks.OnTextureShareGameViewportDraw().Broadcast(*ObjectIt.Value);
			}
		}
	}
}

void FTextureShareAPI::OnGameViewportEndDraw()
{
	check(IsInGameThread());

	for (TPair<FString, TSharedPtr<FTextureShareObject, ESPMode::ThreadSafe>>& ObjectIt : Objects)
	{
		if (ObjectIt.Value.IsValid())
		{
			// Perform callback logic
			if (Callbacks.OnTextureShareGameViewportEndDraw().IsBound())
			{
				Callbacks.OnTextureShareGameViewportEndDraw().Broadcast(*ObjectIt.Value);
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareAPI::OnWorldBeginPlay(UWorld& InWorld)
{ }

void FTextureShareAPI::OnWorldEndPlay(UWorld& InWorld)
{ }

//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareAPI::RegisterRendererModuleCallbacks()
{
	if (!ResolvedSceneColorCallbackHandle.IsValid())
	{
		if (IRendererModule* RendererModule = FModuleManager::GetModulePtr<IRendererModule>(RendererModuleName))
		{
			UE_LOGF(LogTextureShare, Verbose, "Add Renderer module callbacks");
			ResolvedSceneColorCallbackHandle = RendererModule->GetResolvedSceneColorCallbacks().AddRaw(this, &FTextureShareAPI::OnResolvedSceneColor_RenderThread);
		}
	}

	if (!OnBackBufferReadyToPresentHandle.IsValid())
	{
		if (FSlateApplication::IsInitialized())
		{
			OnBackBufferReadyToPresentHandle = FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().AddRaw(this, &FTextureShareAPI::OnBackBufferReadyToPresent_RenderThread);
		}
	}

	if (GEngine && GEngine->GameViewport)
	{
		if (!OnGameViewportBeginDrawHandle.IsValid())
		{
			OnGameViewportBeginDrawHandle = GEngine->GameViewport->OnBeginDraw().AddRaw(this, &FTextureShareAPI::OnGameViewportBeginDraw);
		}

		if (!OnGameViewportDrawHandle.IsValid())
		{
			OnGameViewportDrawHandle = GEngine->GameViewport->OnBeginDraw().AddRaw(this, &FTextureShareAPI::OnGameViewportDraw);
		}

		if (!OnGameViewportEndDrawHandle.IsValid())
		{
			OnGameViewportEndDrawHandle = GEngine->GameViewport->OnEndDraw().AddRaw(this, &FTextureShareAPI::OnGameViewportEndDraw);
		}
	}
}

void FTextureShareAPI::UnregisterRendererModuleCallbacks()
{
	if (ResolvedSceneColorCallbackHandle.IsValid())
	{
		if (IRendererModule* RendererModule = FModuleManager::GetModulePtr<IRendererModule>(RendererModuleName))
		{
			UE_LOGF(LogTextureShare, Verbose, "Remove Renderer module callbacks");
			RendererModule->GetResolvedSceneColorCallbacks().Remove(ResolvedSceneColorCallbackHandle);
		}

		ResolvedSceneColorCallbackHandle.Reset();
	}

	if (OnBackBufferReadyToPresentHandle.IsValid())
	{
		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().Remove(OnBackBufferReadyToPresentHandle);
		}
	}

	if (GEngine && GEngine->GameViewport)
	{
		if (OnGameViewportBeginDrawHandle.IsValid())
		{
			GEngine->GameViewport->OnBeginDraw().Remove(OnGameViewportBeginDrawHandle);
		}

		if (OnGameViewportDrawHandle.IsValid())
		{
			GEngine->GameViewport->OnBeginDraw().Remove(OnGameViewportDrawHandle);
		}

		if (OnGameViewportEndDrawHandle.IsValid())
		{
			GEngine->GameViewport->OnEndDraw().Remove(OnGameViewportEndDrawHandle);
		}
	}
}
