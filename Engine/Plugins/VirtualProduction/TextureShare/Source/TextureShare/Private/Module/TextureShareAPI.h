// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ITextureShareAPI.h"
#include "TextureShareCallbacks.h"

#include "Object/TextureShareObject.h"
#include "Object/TextureShareObjectProxy.h"

#include "Slate/SlateViewportProvider.h"

class SWindow;

/**
 * TextureShare API impl
 */
class FTextureShareAPI
	: public ITextureShareAPI
{
public:
	FTextureShareAPI();
	virtual ~FTextureShareAPI();

public:
	//~ ITextureShareAPI
	virtual TSharedPtr<ITextureShareObject, ESPMode::ThreadSafe> GetOrCreateObject(const FString& ShareName, const ETextureShareProcessType InProcessType = ETextureShareProcessType::Undefined) override;
	virtual bool RemoveObject(const FString& ShareName) override;
	virtual bool IsObjectExist(const FString& ShareName) const override;

	virtual TSharedPtr<ITextureShareObject, ESPMode::ThreadSafe> GetObject(const FString& ShareName) const override;
	virtual TSharedPtr<ITextureShareObjectProxy, ESPMode::ThreadSafe> GetObjectProxy_RenderThread(const FString& ShareName) const override;

	virtual TSharedPtr<ITextureShareObject, ESPMode::ThreadSafe> GetObject(const TSharedRef<const FSceneViewExtensionBase, ESPMode::ThreadSafe>& InViewExtension) const override;
	virtual TSharedPtr<ITextureShareObjectProxy, ESPMode::ThreadSafe> GetObjectProxy_RenderThread(const TSharedRef<const FSceneViewExtensionBase, ESPMode::ThreadSafe>& InViewExtension) const override;


	virtual bool GetInterprocessObjects(const FString& InShareName, TArray<struct FTextureShareCoreObjectDesc>& OutInterprocessObjects) const override;

	virtual const struct FTextureShareCoreObjectProcessDesc& GetProcessDesc() const override;
	virtual void SetProcessName(const FString& InProcessId) override;

	virtual void OnWorldBeginPlay(UWorld& InWorld) override;
	virtual void OnWorldEndPlay(UWorld& InWorld) override;

	virtual ITextureShareCallbacks& GetCallbacks() override
	{
		return Callbacks;
	}
	//~~ ITextureShare

public:
	

private:
	void RemoveTextureShareObjectInstances();

private:
	/** Rendered callback (get scene textures to share) */
	void OnResolvedSceneColor_RenderThread(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures);

	/** Slate app callback before present to share app backbuffer */
	void OnBackBufferReadyToPresent_RenderThread(SWindow& Window, ISlateViewportProvider& ViewportProvider);

	/** GameViewport event onBeginDraw. */
	void OnGameViewportBeginDraw();
	
	/** GameViewport event onDraw. */
	void OnGameViewportDraw();

	/** GameViewport event onEndDraw. */
	void OnGameViewportEndDraw();

	void RegisterRendererModuleCallbacks();
	void UnregisterRendererModuleCallbacks();

#if WITH_EDITOR
	void RegisterSettings_Editor();
	void UnregisterSettings_Editor();
#endif

private:
	TMap<FString, TSharedPtr<FTextureShareObject, ESPMode::ThreadSafe>>      Objects;
	TMap<FString, TSharedPtr<FTextureShareObjectProxy, ESPMode::ThreadSafe>> ObjectProxies;

	FDelegateHandle ResolvedSceneColorCallbackHandle;
	FDelegateHandle OnBackBufferReadyToPresentHandle;

	FDelegateHandle OnGameViewportBeginDrawHandle;
	FDelegateHandle OnGameViewportDrawHandle;
	FDelegateHandle OnGameViewportEndDrawHandle;

	mutable FCriticalSection ThreadDataCS;

	FTextureShareCallbacks Callbacks;
};
