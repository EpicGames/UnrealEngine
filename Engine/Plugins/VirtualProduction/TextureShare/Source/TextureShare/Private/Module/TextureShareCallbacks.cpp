// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/TextureShareCallbacks.h"

#include "ITextureShare.h"
#include "ITextureShareAPI.h"
#include "ITextureShareCallbacks.h"

ITextureShareCallbacks& ITextureShareCallbacks::Get()
{
	static ITextureShareCallbacks& TSCallbacksAPI = ITextureShare::Get().GetTextureShareAPI().GetCallbacks();

	return TSCallbacksAPI;
}
