// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITextureShareCallbacks.h"

/**
 * TextureShare callbacks API implementation
 */
class FTextureShareCallbacks :
	public  ITextureShareCallbacks
{
public:
	virtual ~FTextureShareCallbacks() = default;

public:
	virtual FTextureShareGameViewportBeginDrawEvent& OnTextureShareGameViewportBeginDraw() override
	{
		return TextureShareGameViewportBeginDrawEvent;
	}

	virtual FTextureShareGameViewportDrawEvent& OnTextureShareGameViewportDraw() override
	{
		return TextureShareGameViewportDrawEvent;
	}

	virtual FTextureShareGameViewportEndDrawEvent& OnTextureShareGameViewportEndDraw() override
	{
		return TextureShareGameViewportEndDrawEvent;
	}

	virtual FTextureShareBeginRenderViewFamilyEvent& OnTextureShareBeginRenderViewFamily() override
	{
		return TextureShareBeginRenderViewFamilyEvent;
	}

	virtual FTextureShareBeginSessionEvent& OnTextureShareBeginSession() override
	{
		return TextureShareBeginSessionEvent;
	}
	
	virtual FTextureShareEndSessionEvent& OnTextureShareEndSession() override
	{
		return TextureShareEndSessionEvent;
	}


	virtual FTextureSharePreBeginFrameSyncEvent& OnTextureSharePreBeginFrameSync() override
	{
		return TextureSharePreBeginFrameSyncEvent;
	}

	virtual FTextureShareBeginFrameSyncEvent& OnTextureShareBeginFrameSync() override
	{
		return TextureShareBeginFrameSyncEvent;
	}

	virtual FTextureShareEndFrameSyncEvent& OnTextureShareEndFrameSync() override
	{
		return TextureShareEndFrameSyncEvent;
	}

	virtual FTextureShareFrameSyncEvent& OnTextureShareFrameSync() override
	{
		return TextureShareFrameSyncEvent;
	}

	virtual FTextureSharePreBeginFrameSyncEvent_RenderThread& OnTextureSharePreBeginFrameSync_RenderThread() override
	{
		return TextureSharePreBeginFrameSyncEvent_RenderThread;
	}

	virtual FTextureShareBeginFrameSyncEvent_RenderThread& OnTextureShareBeginFrameSync_RenderThread() override
	{
		return TextureShareBeginFrameSyncEvent_RenderThread;
	}

	virtual FTextureShareEndFrameSyncEvent_RenderThread& OnTextureShareEndFrameSync_RenderThread() override
	{
		return TextureShareEndFrameSyncEvent_RenderThread;
	}

	virtual FTextureShareFrameSyncEvent_RenderThread& OnTextureShareFrameSync_RenderThread() override
	{
		return TextureShareFrameSyncEvent_RenderThread;
	}

	virtual FTextureSharePreRenderViewFamily_RenderThread& OnTextureSharePreRenderViewFamily_RenderThread() override
	{
		return TextureSharePreRenderViewFamily_RenderThread;
	}

	virtual FTextureSharePostRenderViewFamily_RenderThread& OnTextureSharePostRenderViewFamily_RenderThread() override
	{
		return TextureSharePostRenderViewFamily_RenderThread;
	}

	virtual FTextureShareBackBufferReadyToPresentEvent_RenderThread& OnTextureShareBackBufferReadyToPresent_RenderThread() override
	{
		return TextureShareBackBufferReadyToPresentEvent_RenderThread;
	}

private:
	FTextureShareGameViewportBeginDrawEvent TextureShareGameViewportBeginDrawEvent;
	FTextureShareGameViewportDrawEvent      TextureShareGameViewportDrawEvent;
	FTextureShareGameViewportEndDrawEvent   TextureShareGameViewportEndDrawEvent;

	FTextureShareBeginRenderViewFamilyEvent TextureShareBeginRenderViewFamilyEvent;

	FTextureShareBeginSessionEvent     TextureShareBeginSessionEvent;
	FTextureShareEndSessionEvent       TextureShareEndSessionEvent;

	FTextureSharePreBeginFrameSyncEvent   TextureSharePreBeginFrameSyncEvent;
	FTextureShareBeginFrameSyncEvent   TextureShareBeginFrameSyncEvent;

	FTextureShareEndFrameSyncEvent     TextureShareEndFrameSyncEvent;
	FTextureShareFrameSyncEvent        TextureShareFrameSyncEvent;

	FTextureSharePreBeginFrameSyncEvent_RenderThread TextureSharePreBeginFrameSyncEvent_RenderThread;
	FTextureShareBeginFrameSyncEvent_RenderThread TextureShareBeginFrameSyncEvent_RenderThread;

	FTextureShareEndFrameSyncEvent_RenderThread   TextureShareEndFrameSyncEvent_RenderThread;
	FTextureShareFrameSyncEvent_RenderThread      TextureShareFrameSyncEvent_RenderThread;

	FTextureSharePreRenderViewFamily_RenderThread TextureSharePreRenderViewFamily_RenderThread;
	FTextureSharePostRenderViewFamily_RenderThread TextureSharePostRenderViewFamily_RenderThread;
	FTextureShareBackBufferReadyToPresentEvent_RenderThread TextureShareBackBufferReadyToPresentEvent_RenderThread;
};
