// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/TextureShareLog.h"

DEFINE_LOG_CATEGORY(LogTextureShare);
DEFINE_LOG_CATEGORY(LogTextureShareObject);
DEFINE_LOG_CATEGORY(LogTextureShareObjectProxy);
DEFINE_LOG_CATEGORY(LogTextureShareWorldSubsystem);
DEFINE_LOG_CATEGORY(LogTextureShareResource);
