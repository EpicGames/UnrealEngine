// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

#if TEXTURESHARE_DEBUGLOG
// Enable extra log
#define UE_TS_LOG(CategoryName, Verbosity, Format, ...)\
	UE_LOG(CategoryName, Verbosity, Format, ##__VA_ARGS__)

#else
// Disable extra log
#define UE_TS_LOG(CategoryName, Verbosity, Format, ...)
#endif

// Plugin-wide log categories
#if UE_BUILD_SHIPPING
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShare,    Warning, Warning);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareObject, Warning, Warning);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareObjectProxy, Warning, Warning);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareWorldSubsystem,  Warning, Warning);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareResource, Warning, Warning);

#else
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShare,    Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareObject, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareObjectProxy, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareWorldSubsystem,  Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareResource, Log, All);

#endif
