// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/TextureShareModule.h"
#include "Module/TextureShareAPI.h"
#include "Module/TextureShareLog.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "ShaderCore.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// IModuleInterface
//////////////////////////////////////////////////////////////////////////////////////////////

#define TEXTURESHARE_SHADERS_MAP TEXT("/Plugin/TextureShare")

void FTextureShareModule::StartupModule()
{
#if WITH_EDITOR
	RegisterSettings_Editor();
#endif

	if (!AllShaderSourceDirectoryMappings().Contains(TEXTURESHARE_SHADERS_MAP))
	{
		FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("TextureShare"))->GetBaseDir(), TEXT("Shaders"));
		AddShaderSourceDirectoryMapping(TEXTURESHARE_SHADERS_MAP, PluginShaderDir);
	}

	UE_LOGF(LogTextureShare, Log, "TextureShare module has started");
}

void FTextureShareModule::ShutdownModule()
{
#if WITH_EDITOR
	if (UObjectInitialized())
	{
		UnregisterSettings_Editor();
	}
#endif

	TextureShareAPI.Reset();

	UE_LOGF(LogTextureShare, Log, "TextureShare module shutdown");
}

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareModule
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareModule::FTextureShareModule()
{
	UE_LOGF(LogTextureShare, Log, "TextureShare module has been instantiated");
}

FTextureShareModule::~FTextureShareModule()
{
	UE_LOGF(LogTextureShare, Log, "TextureShare module has been destroyed");
}

ITextureShareAPI& FTextureShareModule::GetTextureShareAPI()
{
	if (!TextureShareAPI.IsValid())
	{
		TextureShareAPI = MakeUnique<FTextureShareAPI>();
	}

	check(TextureShareAPI.IsValid());

	return *TextureShareAPI.Get();
}

IMPLEMENT_MODULE(FTextureShareModule, TextureShare);
