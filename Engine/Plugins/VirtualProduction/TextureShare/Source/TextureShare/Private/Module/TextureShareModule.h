// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ITextureShare.h"

/**
 * TextureShare module impl
 */
class FTextureShareModule
	: public ITextureShare
{
public:
	FTextureShareModule();
	virtual ~FTextureShareModule();

public:
	// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~IModuleInterface

public:
	// ITextureShare
	virtual class ITextureShareAPI& GetTextureShareAPI() override;
	//~ITextureShare

private:
	TUniquePtr<class FTextureShareAPI> TextureShareAPI;

private:
#if WITH_EDITOR
	void RegisterSettings_Editor();
	void UnregisterSettings_Editor();
#endif
};
