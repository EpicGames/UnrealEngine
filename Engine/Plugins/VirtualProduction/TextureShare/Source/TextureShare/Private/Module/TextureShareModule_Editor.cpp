// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/TextureShareModule.h"

#if WITH_EDITOR
#include "Module/TextureShareLog.h"
#include "Misc/TextureShareStrings.h"

#include "ISettingsModule.h"
#include "Game/Settings/TextureShareSettings.h"

//////////////////////////////////////////////////////////////////////////////////////////////
#define LOCTEXT_NAMESPACE "TextureShare"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareModule
//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareModule::RegisterSettings_Editor()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->RegisterSettings(
			UTextureShareSettings::Container,
			UTextureShareSettings::Category,
			UTextureShareSettings::Section,
			LOCTEXT("RuntimeSettingsName", "TextureShare"),
			LOCTEXT("RuntimeSettingsDescription", "Configure TextureShare"),
			GetMutableDefault<UTextureShareSettings>()
		);
	}
}
void FTextureShareModule::UnregisterSettings_Editor()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings(UTextureShareSettings::Container, UTextureShareSettings::Category, UTextureShareSettings::Section);
	}
}

#undef LOCTEXT_NAMESPACE 

#endif
