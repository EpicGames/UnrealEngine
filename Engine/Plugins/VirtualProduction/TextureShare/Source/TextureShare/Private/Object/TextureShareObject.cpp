// Copyright Epic Games, Inc. All Rights Reserved.

#include "Object/TextureShareObject.h"
#include "Object/TextureShareObjectProxy.h"
#include "Module/TextureShareLog.h"
#include "Misc/TextureShareStrings.h"
#include "Core/TextureShareCoreHelpers.h"

#include "ITextureShareCallbacks.h"
#include "ITextureShareCoreObject.h"

using namespace UE::TextureShareCore;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareObject
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareObject::FTextureShareObject(const TSharedRef<ITextureShareCoreObject, ESPMode::ThreadSafe>& InCoreObject)
	: CoreObject(InCoreObject)
	, ObjectProxy(new FTextureShareObjectProxy(CoreObject))
	, TextureShareData(MakeShared<FTextureShareData, ESPMode::ThreadSafe>())
{ }

FTextureShareObject::~FTextureShareObject()
{
	EndSession();
}

void FTextureShareObject::SetTextureShareContext(const TSharedPtr<ITextureShareContext, ESPMode::ThreadSafe>& InTextureShareContext)
{
	TextureShareContext = InTextureShareContext;

	// Register TS context callbacks on the game thread.
	if (TextureShareContext.IsValid())
	{
		TextureShareContext->RegisterTextureShareContextCallbacks();
	}
}

const ITextureShareContext* FTextureShareObject::GetTextureShareContext() const
{
	return TextureShareContext.Get();
}

//////////////////////////////////////////////////////////////////////////////////////////////
const FString& FTextureShareObject::GetName() const
{
	return CoreObject->GetName();
}

FTextureShareCoreObjectDesc FTextureShareObject::GetObjectDesc() const
{
	return CoreObject->GetObjectDesc();
}

bool FTextureShareObject::IsActive() const
{
	return CoreObject->IsActive();
}

bool FTextureShareObject::IsFrameSyncActive() const
{
	return bFrameSyncActive && CoreObject->IsFrameSyncActive();
}

bool FTextureShareObject::SetProcessId(const FString& InProcessId)
{
	return CoreObject->SetProcessId(InProcessId);
}

bool FTextureShareObject::SetSyncSetting(const FTextureShareCoreSyncSettings& InSyncSetting)
{
	return CoreObject->SetSyncSettings(InSyncSetting);
}

FTextureShareCoreSyncSettings FTextureShareObject::GetSyncSetting() const
{
	return CoreObject->GetSyncSettings();
}

FTextureShareCoreFrameSyncSettings FTextureShareObject::GetFrameSyncSettings(const ETextureShareFrameSyncTemplate InType) const
{
	return CoreObject->GetFrameSyncSettings(InType);
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareObject::BeginSession(FViewport* InViewport)
{
	if (!bSessionActive && CoreObject->BeginSession())
	{
		bSessionActive = true;

		UE_TS_LOG(LogTextureShareObject, Log, TEXT("%s:BeginSession"), *GetName());

		FTextureShareObjectProxy::BeginSession_GameThread(*this);

		if (InViewport)
		{
			UpdateViewExtension(InViewport);
		}

		if (ITextureShareCallbacks::Get().OnTextureShareBeginSession().IsBound())
		{
			ITextureShareCallbacks::Get().OnTextureShareBeginSession().Broadcast(*this);
		}

		return true;
	}

	return false;
}

bool FTextureShareObject::EndSession()
{
	if (bSessionActive)
	{
		bSessionActive = false;

		UE_TS_LOG(LogTextureShareObject, Log, TEXT("%s:EndSession"), *GetName());

		FTextureShareObjectProxy::EndSession_GameThread(*this);

		if (ITextureShareCallbacks::Get().OnTextureShareEndSession().IsBound())
		{
			ITextureShareCallbacks::Get().OnTextureShareEndSession().Broadcast(*this);
		}

		return true;
	}

	return false;
}

bool FTextureShareObject::IsSessionActive() const
{
	return CoreObject->IsSessionActive();
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareObject::BeginFrameSync()
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShare::BeginFrameSync(%s)"), *GetName()));

	if (ITextureShareCallbacks::Get().OnTextureSharePreBeginFrameSync().IsBound())
	{
		ITextureShareCallbacks::Get().OnTextureSharePreBeginFrameSync().Broadcast(*this);
	}

	// When starting a new frame in the game thread, we must inform about it the proxy in the rendering thread.
	FTextureShareObjectProxy::OnTextureSharePreBeginFrameSync_GameThread(*this);

	if (!CoreObject->IsBeginFrameSyncActive())
	{
		return false;
	}

	UE_TS_LOG(LogTextureShareObject, Log, TEXT("%s:BeginFrameSync"), *GetName());

	if (!CoreObject->LockThreadMutex(ETextureShareThreadMutex::GameThread))
	{
		return false;
	}

	if (!CoreObject->BeginFrameSync())
	{
		CoreObject->UnlockThreadMutex(ETextureShareThreadMutex::GameThread);

		return false;
	}

	// TS processes is connected now
	bFrameSyncActive = true;

	if (ITextureShareCallbacks::Get().OnTextureShareBeginFrameSync().IsBound())
	{
		ITextureShareCallbacks::Get().OnTextureShareBeginFrameSync().Broadcast(*this);
	}

	return true;
}

bool FTextureShareObject::EndFrameSync()
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShare::EndFrameSync(%s)"), *GetName()));

	if (!IsFrameSyncActive())
	{
		UE_TS_LOG(LogTextureShareObject, Log, TEXT("%s:EndFrameSync: Canceled"), *GetName());

		bFrameSyncActive = false;

		// Reset ptr to data, now this data used in proxy
		TextureShareData = MakeShared<FTextureShareData, ESPMode::ThreadSafe>();

		CoreObject->UnlockThreadMutex(ETextureShareThreadMutex::GameThread);

		return false;
	}

	// Always force flush sync at the frame end
	FrameSync(ETextureShareSyncStep::FrameFlush);

	UE_TS_LOG(LogTextureShareObject, Log, TEXT("%s:EndFrameSync"), *GetName());

	// Game thread data now sent to the proxy.Game-thread data can now be cleared
	const bool bResult = CoreObject->EndFrameSync();

	if (ITextureShareCallbacks::Get().OnTextureShareEndFrameSync().IsBound())
	{
		ITextureShareCallbacks::Get().OnTextureShareEndFrameSync().Broadcast(*this);
	}

	bFrameSyncActive = false;

	// Store gamethread data for proxy
	TextureShareData->ObjectData = CoreObject->GetData();
	TextureShareData->ReceivedObjectsData.Append(CoreObject->GetReceivedData());

	FTextureShareObjectProxy::UpdateProxy_GameThread(*this);

	// Reset ptr to data, now this data used in proxy
	TextureShareData = MakeShared<FTextureShareData, ESPMode::ThreadSafe>();

	CoreObject->UnlockThreadMutex(ETextureShareThreadMutex::RenderingThread);

	return bResult;
}

bool FTextureShareObject::FrameSync(const ETextureShareSyncStep InSyncStep)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShare::FrameSync(%s, %s)"), *GetName(), GetTEXT(InSyncStep)));

	if (IsFrameSyncActive())
	{
		UE_TS_LOG(LogTextureShareObject, Log, TEXT("%s:FrameSync(%s)"), *GetName(), GetTEXT(InSyncStep));

		// Recall all skipped sync steps
		ETextureShareSyncStep SkippedSyncStep;
		while (CoreObject->FindSkippedSyncStep(InSyncStep, SkippedSyncStep))
	{
			if (!DoFrameSync(SkippedSyncStep))
		{
				break;
			}
		}

		// call requested syncstep
		if (DoFrameSync(InSyncStep))
		{
		return true;
	}

		CoreObject->UnlockThreadMutex(ETextureShareThreadMutex::GameThread);
	}

	UE_TS_LOG(LogTextureShareObject, Error, TEXT("%s:FrameSync(%s) failed"), *GetName(), GetTEXT(InSyncStep));

	return false;
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareObject::DoFrameSync(const ETextureShareSyncStep InSyncStep)
{
	UE_TS_LOG(LogTextureShareObject, Log, TEXT("%s:DoFrameSync(%s)"), *GetName(), GetTEXT(InSyncStep));

	if(CoreObject->FrameSync(InSyncStep))
	{
		if (ITextureShareCallbacks::Get().OnTextureShareFrameSync().IsBound())
		{
			ITextureShareCallbacks::Get().OnTextureShareFrameSync().Broadcast(*this, InSyncStep);
		}

		return true;
	}

	return false;
}

//////////////////////////////////////////////////////////////////////////////////////////////
TArray<FTextureShareCoreObjectDesc> FTextureShareObject::GetConnectedInterprocessObjects() const
{
	return CoreObject->GetConnectedInterprocessObjects();
}

FTextureShareCoreData& FTextureShareObject::GetCoreData()
{
	return CoreObject->GetData();
}

const FTextureShareCoreData& FTextureShareObject::GetCoreData() const
{
	return CoreObject->GetData();
}

TArray<FTextureShareCoreObjectData> FTextureShareObject::GetReceivedCoreObjectData() const
{
	return CoreObject->GetReceivedData();
}

//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareData& FTextureShareObject::GetData()
{
	return *TextureShareData;
}

TSharedPtr<FTextureShareSceneViewExtension, ESPMode::ThreadSafe> FTextureShareObject::GetViewExtension() const
{
	return ViewExtension;
}

TSharedRef<ITextureShareObjectProxy, ESPMode::ThreadSafe> FTextureShareObject::GetProxy() const
{
	return ObjectProxy;
}

void FTextureShareObject::UpdateViewExtension(FViewport* InViewport)
{
	check(InViewport);

	if (!ViewExtension.IsValid() || ViewExtension->LinkedViewport != InViewport)
	{
		// create new one
		ViewExtension = FSceneViewExtensions::NewExtension<FTextureShareSceneViewExtension>(InViewport);
	}
}
