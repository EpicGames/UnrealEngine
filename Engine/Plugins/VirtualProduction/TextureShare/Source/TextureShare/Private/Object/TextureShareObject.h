// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ITextureShareObject.h"
#include "ITextureShareCoreObject.h"

#include "Game/ViewExtension/TextureShareSceneViewExtension.h"
#include "Templates/SharedPointer.h"

class FViewport;
class FTextureShareObjectProxy;

/**
 * TextureShare object
 */
class FTextureShareObject
	: public ITextureShareObject
	, public TSharedFromThis<FTextureShareObject, ESPMode::ThreadSafe>
{
public:
	FTextureShareObject(const TSharedRef<ITextureShareCoreObject, ESPMode::ThreadSafe>& InCoreObject);
	virtual ~FTextureShareObject();
	
public:
	// ~ITextureShareObject
	virtual void SetTextureShareContext(const TSharedPtr<ITextureShareContext, ESPMode::ThreadSafe>& InTextureShareContext) override;
	virtual const ITextureShareContext* GetTextureShareContext() const  override;

	virtual const FString& GetName() const override;
	virtual FTextureShareCoreObjectDesc GetObjectDesc() const override;

	virtual bool IsActive() const override;
	virtual bool IsFrameSyncActive() const override;

	virtual bool SetProcessId(const FString& InProcessId) override;
	
	virtual bool SetSyncSetting(const FTextureShareCoreSyncSettings& InSyncSetting) override;
	virtual FTextureShareCoreSyncSettings GetSyncSetting() const override;

	virtual FTextureShareCoreFrameSyncSettings GetFrameSyncSettings(const ETextureShareFrameSyncTemplate InType) const override;

	virtual bool BeginSession(FViewport* InViewport) override;
	virtual bool EndSession() override;
	virtual bool IsSessionActive() const override;

	virtual bool BeginFrameSync() override;
	virtual bool FrameSync(const ETextureShareSyncStep InSyncStep) override;
	virtual bool EndFrameSync() override;

	virtual TArray<FTextureShareCoreObjectDesc> GetConnectedInterprocessObjects() const override;

	virtual FTextureShareCoreData& GetCoreData() override;
	virtual const FTextureShareCoreData& GetCoreData() const override;

	virtual TArray<FTextureShareCoreObjectData> GetReceivedCoreObjectData() const override;
	
	virtual FTextureShareData& GetData() override;

	virtual TSharedPtr<FTextureShareSceneViewExtension, ESPMode::ThreadSafe> GetViewExtension() const override;

	virtual TSharedRef<ITextureShareObjectProxy, ESPMode::ThreadSafe> GetProxy() const override;
	//~~ITextureShareObject

public:
	const TSharedRef<FTextureShareObjectProxy, ESPMode::ThreadSafe>& GetObjectProxyRef() const
	{
		return ObjectProxy;
	}

private:
	void UpdateViewExtension(FViewport* InViewport);

	/**
	 * Implementing the synchronization step (calling a function from TextureShareCore) and handling callbacks
	 * This function can be called multiple times from FrameSync() to add missing synchronization steps.
	 */
	bool DoFrameSync(const ETextureShareSyncStep InSyncStep);


protected:
	friend class FTextureShareObjectProxy;

	// TS Core lib coreobject
	const TSharedRef<ITextureShareCoreObject, ESPMode::ThreadSafe> CoreObject;

	// Render thread object proxy
	const TSharedRef<FTextureShareObjectProxy, ESPMode::ThreadSafe> ObjectProxy;

	// Object data from game thread
	TSharedRef<FTextureShareData, ESPMode::ThreadSafe> TextureShareData;

	// Is an abstract container that can be used by the user to handle callback logic.
	TSharedPtr<ITextureShareContext, ESPMode::ThreadSafe> TextureShareContext;

	bool bFrameSyncActive = false;
	bool bSessionActive = false;

	TSharedPtr<FTextureShareSceneViewExtension, ESPMode::ThreadSafe> ViewExtension;
};
