// Copyright Epic Games, Inc. All Rights Reserved.

#include "Object/TextureShareObjectProxy.h"
#include "Object/TextureShareObject.h"

#include "Game/ViewExtension/TextureShareSceneViewExtension.h"
#include "Resources/TextureShareResourcesProxy.h"

#include "Module/TextureShareLog.h"
#include "Misc/TextureShareStrings.h"
#include "Core/TextureShareCoreHelpers.h"

#include "ITextureShareCallbacks.h"
#include "ITextureShareCore.h"

#include "RenderGraphUtils.h"
#include "Async/Async.h"
#include "TextureResource.h"

using namespace UE::TextureShareCore;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareObjectProxy
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareObjectProxy::FTextureShareObjectProxy(const TSharedRef<ITextureShareCoreObject, ESPMode::ThreadSafe>& InCoreObject)
	: CoreObject(InCoreObject)
	, TextureShareData(MakeShared<FTextureShareData, ESPMode::ThreadSafe>())
{ }

FTextureShareObjectProxy::~FTextureShareObjectProxy()
{
	EndSession_RenderThread();

	// Release should be on the both threads.
	AssignNewContext_RenderThread(nullptr);
	AssignNewViewExtension_RenderThread(nullptr);
	ReleaseCoreObject_RenderThread();
}

void FTextureShareObjectProxy::AssignNewContext_RenderThread(const TSharedPtr<ITextureShareContext, ESPMode::ThreadSafe>& InTextureShareContext)
{
	if (TextureShareContext == InTextureShareContext)
	{
		// if the context is the same, do nothing.
		return;
	}

	if (TextureShareContext.IsValid())
	{
		// Unregister old context callbacks on the rendering thread
		TextureShareContext->UnregisterTextureShareContextCallbacks_RenderThread();
	}

	// Release old context on the game thread
	AsyncTask(
		ENamedThreads::GameThread_Local,
		[TextureShareContext = TextureShareContext]()
		{
			// Wait for the rendering thread to finish so that the last TSharedPtr<> is in the game thread.
			while ((TextureShareContext.IsValid() && !TextureShareContext.IsUnique()))
			{
				// 1ms
				FPlatformProcess::SleepNoStats(0.0001f);
			}
		});

	// Set new context
	TextureShareContext = InTextureShareContext;

	if (TextureShareContext.IsValid())
	{
		// Register callbacks for the new context on the rendering thread
		TextureShareContext->RegisterTextureShareContextCallbacks_RenderThread();
	}
}

/** This function assigns a new ViewExtension to the proxy object and handles multithreading issues. */
void FTextureShareObjectProxy::AssignNewViewExtension_RenderThread(const TSharedPtr<FTextureShareSceneViewExtension, ESPMode::ThreadSafe>& InViewExtension)
{
	if (ViewExtension == InViewExtension)
	{
		// if the VE is the same, do nothing.
		return;
	}

	// Marks the old VE as unused.
	if (ViewExtension.IsValid())
	{
		ViewExtension->Release_RenderThread();
	}

	// Release VE on the game thread
	AsyncTask(
		ENamedThreads::GameThread_Local,
		[ViewExtension = ViewExtension]()
		{
			// Wait for the rendering thread to finish so that the last TSharedPtr<> is in the game thread.
			while ((ViewExtension.IsValid() && !ViewExtension.IsUnique()))
			{
				// 1ms
				FPlatformProcess::SleepNoStats(0.0001f);
			}
		});

	// Set new VE
	ViewExtension = InViewExtension;
}

/** The CoreObject should be destroyed in the game thread. */
void FTextureShareObjectProxy::ReleaseCoreObject_RenderThread()
{
	// Release CoreObject only on the game thread.
	AsyncTask(
		ENamedThreads::GameThread_Local,
		[CoreObject = CoreObject]()
		{
			// Now the TextureShareContext destructor is called in the game thread

			// Finally release CoreObject
			CoreObject->RemoveObject();
		});
}

//////////////////////////////////////////////////////////////////////////////////////////////
const ITextureShareContext* FTextureShareObjectProxy::GetTextureShareContext_RenderThread() const
{
	return TextureShareContext.Get();
}

ETextureShareObjectProxyFlags FTextureShareObjectProxy::GetObjectProxyFlags() const
{
	ETextureShareObjectProxyFlags OutFlags = ObjectProxyFlags;

	// other flags
	if (ViewExtension.IsValid())
	{
		EnumAddFlags(OutFlags, ETextureShareObjectProxyFlags::ViewExtensionUsed);
	}

	// return updated flags
	return OutFlags;
}

void FTextureShareObjectProxy::SetObjectProxyFlags(const ETextureShareObjectProxyFlags InFlags)
{
	// Ignore internal flags
	const ETextureShareObjectProxyFlags NewFlags = InFlags & (~ETextureShareObjectProxyFlags::InternalFlags);

	// Internal flags cannot be changed by this function.
	const ETextureShareObjectProxyFlags InternalFlags = ObjectProxyFlags & ETextureShareObjectProxyFlags::InternalFlags;

	// Merge new flags with old internal flags;
	ObjectProxyFlags = NewFlags | InternalFlags;
}

//////////////////////////////////////////////////////////////////////////////////////////////
const FString& FTextureShareObjectProxy::GetName_RenderThread() const
{
	check(IsInRenderingThread());

	return CoreObject->GetName();
}

FTextureShareCoreObjectDesc FTextureShareObjectProxy::GetObjectDesc_RenderThread() const
{
	check(IsInRenderingThread());

	return CoreObject->GetObjectDesc_RenderThread();
}

bool FTextureShareObjectProxy::IsActive_RenderThread() const
{
	check(IsInRenderingThread());

	return CoreObject->IsActive_RenderThread();
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareObjectProxy::IsFrameSyncActive_RenderThread() const
{
	check(IsInRenderingThread());

	return EnumHasAllFlags(ObjectProxyFlags, ETextureShareObjectProxyFlags::SessionStarted | ETextureShareObjectProxyFlags::FrameProxySyncActive)
		&& CoreObject->IsFrameSyncActive_RenderThread();
}

bool FTextureShareObjectProxy::BeginFrameSync_RenderThread(FRHICommandListImmediate& RHICmdList) const
{
	check(IsInRenderingThread());

	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShare::BeginFrameSync_RenderThread(%s)"), *GetName_RenderThread()));

	if (!CoreObject->IsBeginFrameSyncActive_RenderThread())
	{
		EnumRemoveFlags(ObjectProxyFlags, ETextureShareObjectProxyFlags::FrameProxySyncActive);
		UE_TS_LOG(LogTextureShareObjectProxy, Error, TEXT("%s:BeginFrameSync_RenderThread() Failed: no active sync"), *GetName_RenderThread());

		return false;
	}

	if (!CoreObject->LockThreadMutex(ETextureShareThreadMutex::RenderingThread))
	{
		EnumRemoveFlags(ObjectProxyFlags, ETextureShareObjectProxyFlags::FrameProxySyncActive);
		UE_TS_LOG(LogTextureShareObjectProxy, Error, TEXT("%s:BeginFrameSync_RenderThread() Failed: Tread mutex failed"), *GetName_RenderThread());

		return false;
	}

	UE_TS_LOG(LogTextureShareObjectProxy, Log, TEXT("%s:BeginFrameSync_RenderThread()"), *GetName_RenderThread());

	if (!CoreObject->BeginFrameSync_RenderThread())
	{
		EnumRemoveFlags(ObjectProxyFlags, ETextureShareObjectProxyFlags::FrameProxySyncActive);
		UE_TS_LOG(LogTextureShareObjectProxy, Error, TEXT("%s:BeginFrameSync_RenderThread() Failed to begin proxy frame sync"), *GetName_RenderThread());

		CoreObject->UnlockThreadMutex(ETextureShareThreadMutex::GameThread);

		return false;
	}

	// update frame markers from game thread data
	FTextureShareCoreProxyData& CoreProxyDataRef = CoreObject->GetProxyData_RenderThread();
				
	// Copy frame marker from game thread
	CoreProxyDataRef.FrameMarker = TextureShareData->ObjectData.FrameMarker;

	// Copy the frame markers from the objects saved at the end of the game stream.
	CoreProxyDataRef.RemoteFrameMarkers.Empty();

	// Update remote frame markers
	for (const FTextureShareCoreObjectData& ObjectDataIt : TextureShareData->ReceivedObjectsData)
	{
		CoreProxyDataRef.RemoteFrameMarkers.Add(FTextureShareCoreObjectFrameMarker(ObjectDataIt.Desc, ObjectDataIt.Data.FrameMarker));
	}

	EnumAddFlags(ObjectProxyFlags, ETextureShareObjectProxyFlags::FrameProxySyncActive);

	UE_TS_LOG(LogTextureShareObjectProxy, Log, TEXT("%s:BeginFrameSync_RenderThread() Completed"), *GetName_RenderThread());

	if (ITextureShareCallbacks::Get().OnTextureShareBeginFrameSync_RenderThread().IsBound())
	{
		ITextureShareCallbacks::Get().OnTextureShareBeginFrameSync_RenderThread().Broadcast(RHICmdList, *this);
	}

	return true;
}

bool FTextureShareObjectProxy::EndFrameSync_RenderThread(FRHICommandListImmediate& RHICmdList) const
{
	check(IsInRenderingThread());

	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShare::EndFrameSync_RenderThread(%s)"), *GetName_RenderThread()));

	if (!EnumHasAnyFlags(ObjectProxyFlags, ETextureShareObjectProxyFlags::FrameProxySyncActive))
	{
		return false;
	}

	if (!IsFrameSyncActive_RenderThread())
	{
		UE_TS_LOG(LogTextureShareObjectProxy, Error, TEXT("%s:EndFrameSync_RenderThread() Failed: no active sync"), *GetName_RenderThread());

		EnumRemoveFlags(ObjectProxyFlags, ETextureShareObjectProxyFlags::FrameProxySyncActive);

		// Unlock game thread
		CoreObject->UnlockThreadMutex(ETextureShareThreadMutex::GameThread);

		return false;
	}

	UE_TS_LOG(LogTextureShareObjectProxy, Log, TEXT("%s:EndFrameSync_RenderThread()"), *GetName_RenderThread());

	// Always force RHI&sync flush for render proxy
	FrameSync_RenderThread(RHICmdList, ETextureShareSyncStep::FrameProxyFlush);

	const bool bResult = CoreObject->EndFrameSync_RenderThread();

	EnumRemoveFlags(ObjectProxyFlags, ETextureShareObjectProxyFlags::FrameProxySyncActive);

	if (ITextureShareCallbacks::Get().OnTextureShareEndFrameSync_RenderThread().IsBound())
	{
		ITextureShareCallbacks::Get().OnTextureShareEndFrameSync_RenderThread().Broadcast(RHICmdList, *this);
	}

	CoreObject->UnlockThreadMutex(ETextureShareThreadMutex::GameThread);

	return bResult;
}

bool FTextureShareObjectProxy::FrameSync_RenderThread(FRHICommandListImmediate& RHICmdList, const ETextureShareSyncStep InSyncStep) const
{
	check(IsInRenderingThread());

	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShare::FrameSync_RenderThread(%s, %s)"), *GetName_RenderThread(), GetTEXT(InSyncStep)));

	if (!ResourcesProxy.IsValid())
	{
		UE_TS_LOG(LogTextureShareObjectProxy, Error, TEXT("%s:FrameSync_RenderThread(%s) Failed - Resources proxy invalid. Maybe session isn't started"), *GetName_RenderThread(), GetTEXT(InSyncStep));
		
		CoreObject->UnlockThreadMutex(ETextureShareThreadMutex::GameThread);

		return false;
	}

	if (!IsFrameSyncActive_RenderThread())
	{
		UE_TS_LOG(LogTextureShareObjectProxy, Error, TEXT("%s:FrameSync_RenderThread(%s) Failed - No active frame sync"), *GetName_RenderThread(), GetTEXT(InSyncStep));
		
		CoreObject->UnlockThreadMutex(ETextureShareThreadMutex::GameThread);

		return false;
	}

	UE_TS_LOG(LogTextureShareObjectProxy, Log, TEXT("%s:FrameSync_RenderThread(%s)"), *GetName_RenderThread(), GetTEXT(InSyncStep));

	// Recall all skipped sync steps
	ETextureShareSyncStep SkippedSyncStep;
	while (CoreObject->FindSkippedSyncStep_RenderThread(InSyncStep, SkippedSyncStep))
	{
		if (!DoFrameSync_RenderThread(RHICmdList, SkippedSyncStep))
		{
			UE_TS_LOG(LogTextureShareObjectProxy, Error, TEXT("%s:FrameSync_RenderThread(%s) failed handle skipped syncstep '%s'"), *GetName_RenderThread(), GetTEXT(InSyncStep), GetTEXT(SkippedSyncStep));
			
			CoreObject->UnlockThreadMutex(ETextureShareThreadMutex::GameThread);

			return false;
		}
	}

	// call requested syncstep
	if (!DoFrameSync_RenderThread(RHICmdList, InSyncStep))
	{
		UE_TS_LOG(LogTextureShareObjectProxy, Error, TEXT("%s:FrameSync_RenderThread(%s) failed"), *GetName_RenderThread(), GetTEXT(InSyncStep));
		
		CoreObject->UnlockThreadMutex(ETextureShareThreadMutex::GameThread);

		return false;
	}

	return true;
}

DECLARE_GPU_STAT_NAMED(TextureShareObjectProxyFrameSync, TEXT("TextureShare::FrameSync_RenderThread"));
bool FTextureShareObjectProxy::DoFrameSync_RenderThread(FRHICommandListImmediate& RHICmdList, const ETextureShareSyncStep InSyncStep) const
{
	if (!ResourcesProxy.IsValid())
	{
		UE_TS_LOG(LogTextureShareObjectProxy, Error, TEXT("%s:DoFrameSync_RenderThread(%s) Failed - Resources proxy invalid. Maybe session isn't started"), *GetName_RenderThread(), GetTEXT(InSyncStep));

		return false;
	}

	if (!IsFrameSyncActive_RenderThread())
	{
		UE_TS_LOG(LogTextureShareObjectProxy, Error, TEXT("%s:DoFrameSync_RenderThread(%s) Failed - No active frame sync"), *GetName_RenderThread(), GetTEXT(InSyncStep));

		return false;
	}

	UE_TS_LOG(LogTextureShareObjectProxy, Log, TEXT("%s:DoFrameSync_RenderThread(%s)"), *GetName_RenderThread(), GetTEXT(InSyncStep));

	RHI_BREADCRUMB_EVENT_STAT(RHICmdList, TextureShareObjectProxyFrameSync, "TextureShareObjectProxyFrameSync");

	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShare::DoFrameSync_RenderThread(%s, %s)"), *GetName_RenderThread(), GetTEXT(InSyncStep)));

	// step 1: support mGPU for sender
	ResourcesProxy->RunCrossGPUTransfer_RenderThread(ECrossGPUTransferType::BeforeSync, RHICmdList, InSyncStep);

	// step 2: update shared resources handles and register it
	ResourcesProxy->RunRegisterResourceHandles_RenderThread(RHICmdList);

	// step 3: flush RHI thread if needed to be sure about surfaces ready for sharing
	ResourcesProxy->RHIThreadFlush_RenderThread(RHICmdList);

	// step 4: synchronize data between processes
	if (!CoreObject->FrameSync_RenderThread(InSyncStep) || !ResourcesProxy.IsValid())
	{
		UE_TS_LOG(LogTextureShareObjectProxy, Error, TEXT("%s:DoFrameSync_RenderThread(%s) Failed"), *GetName_RenderThread(), GetTEXT(InSyncStep));

		return false;
	}

	// step 5: support mGPU for received textures
	ResourcesProxy->RunCrossGPUTransfer_RenderThread(ECrossGPUTransferType::AfterSync, RHICmdList, InSyncStep);

	// step 6: copy received textures
	ResourcesProxy->RunReceiveResources_RenderThread(RHICmdList, InSyncStep);

	if (ITextureShareCallbacks::Get().OnTextureShareFrameSync_RenderThread().IsBound())
	{
		ITextureShareCallbacks::Get().OnTextureShareFrameSync_RenderThread().Broadcast(RHICmdList, *this, InSyncStep);
	}

	return true;
}

//////////////////////////////////////////////////////////////////////////////////////////////
const FTextureShareData& FTextureShareObjectProxy::GetData_RenderThread() const
{
	check(IsInRenderingThread());

	return *TextureShareData;
}

FTextureShareCoreProxyData& FTextureShareObjectProxy::GetCoreProxyData_RenderThread()
{
	check(IsInRenderingThread());

	return CoreObject->GetProxyData_RenderThread();
}

const FTextureShareCoreProxyData& FTextureShareObjectProxy::GetCoreProxyData_RenderThread() const
{
	check(IsInRenderingThread());

	return CoreObject->GetProxyData_RenderThread();
}

TArray<FTextureShareCoreObjectProxyData> FTextureShareObjectProxy::GetReceivedCoreObjectProxyData_RenderThread() const
{
	check(IsInRenderingThread());

	return CoreObject->GetReceivedProxyData_RenderThread();
}

//////////////////////////////////////////////////////////////////////////////////////////////
BEGIN_SHADER_PARAMETER_STRUCT(FSendTextureParameters, )
RDG_TEXTURE_ACCESS(Texture, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FReceiveTextureParameters, )
RDG_TEXTURE_ACCESS(Texture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

bool FTextureShareObjectProxy::ShareResource_RenderThread(
	FRDGBuilder& GraphBuilder,
	const FTextureShareCoreResourceDesc& InResourceDesc,
	const FRDGTextureRef& InTextureRef,
	const FTextureShareColorDesc& InTextureColorDesc,
	const int32 InTextureGPUIndex,
	const FIntRect* InTextureRect) const
{
	if (HasBeenProduced(InTextureRef))
	{
		FIntRect InViewRect = (InTextureRect) ? *InTextureRect : FIntRect();

		switch (InResourceDesc.OperationType)
		{
		case ETextureShareTextureOp::Read:
		{
			FSendTextureParameters* PassParameters = GraphBuilder.AllocParameters<FSendTextureParameters>();
			PassParameters->Texture = InTextureRef;

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("TextureShare_SendRDGTexture_%s", *InResourceDesc.ResourceName),
				PassParameters,
				ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,
				[ObjectProxy = SharedThis(this), InResourceDesc, InTextureRef, InTextureColorDesc, InTextureGPUIndex, InViewRect](FRHICommandListImmediate& RHICmdList)
				{
					ObjectProxy->ShareResource_RenderThread(RHICmdList, InResourceDesc, InTextureRef->GetRHI(), InTextureColorDesc, InTextureGPUIndex, &InViewRect);
				});

			return true;
		}

		case ETextureShareTextureOp::Write:
		{
			FReceiveTextureParameters* PassParameters = GraphBuilder.AllocParameters<FReceiveTextureParameters>();
			PassParameters->Texture = InTextureRef;

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("TextureShare_ReceiveRDGTexture_%s", *InResourceDesc.ResourceName),
				PassParameters,
				ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,
				[ObjectProxy = SharedThis(this), InResourceDesc, InTextureRef, InTextureColorDesc, InTextureGPUIndex, InViewRect](FRHICommandListImmediate& RHICmdList)
				{
					ObjectProxy->ShareResource_RenderThread(RHICmdList, InResourceDesc, InTextureRef->GetRHI(), InTextureColorDesc, InTextureGPUIndex, &InViewRect);
				});

			return true;
		}

		default:
			break;
		}
	}

	return false;
}

bool FTextureShareObjectProxy::ShareResource_RenderThread(
	FRDGBuilder& GraphBuilder,
	const FTextureShareCoreResourceRequest& InResourceRequest,
	const FRDGTextureRef& InTextureRef,
	const FTextureShareColorDesc& InTextureColorDesc,
	const int32 InTextureGPUIndex,
	const FIntRect* InTextureRect) const
{
	if (HasBeenProduced(InTextureRef))
	{
		FIntRect InViewRect = (InTextureRect) ? *InTextureRect : FIntRect();

		switch (InResourceRequest.ResourceDesc.OperationType)
		{
		case ETextureShareTextureOp::Read:
		{
			FSendTextureParameters* PassParameters = GraphBuilder.AllocParameters<FSendTextureParameters>();
			PassParameters->Texture = InTextureRef;

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("TextureShare_SendRDGTexture_%s", *InResourceRequest.ResourceDesc.ResourceName),
				PassParameters,
				ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,
				[ObjectProxy = SharedThis(this), InResourceRequest, InTextureRef, InTextureColorDesc, InTextureGPUIndex, InViewRect](FRHICommandListImmediate& RHICmdList)
				{
					ObjectProxy->ShareResource_RenderThread(RHICmdList, InResourceRequest, InTextureRef->GetRHI(), InTextureColorDesc, InTextureGPUIndex, &InViewRect);
				});

			return true;
		}

		case ETextureShareTextureOp::Write:
		{
			FReceiveTextureParameters* PassParameters = GraphBuilder.AllocParameters<FReceiveTextureParameters>();
			PassParameters->Texture = InTextureRef;

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("TextureShare_ReceiveRDGTexture_%s", *InResourceRequest.ResourceDesc.ResourceName),
				PassParameters,
				ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,
				[ObjectProxy = SharedThis(this), InResourceRequest, InTextureRef, InTextureColorDesc, InTextureGPUIndex, InViewRect](FRHICommandListImmediate& RHICmdList)
				{
					ObjectProxy->ShareResource_RenderThread(RHICmdList, InResourceRequest, InTextureRef->GetRHI(), InTextureColorDesc, InTextureGPUIndex, &InViewRect);
				});

			return true;
		}

		default:
			break;
		}
	}

	return false;
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareObjectProxy::ShareResource_RenderThread(
	FRHICommandListImmediate& RHICmdList,
	const FTextureShareCoreResourceDesc& InResourceDesc,
	FRHITexture* InTexture,
	const FTextureShareColorDesc& InTextureColorDesc,
	const int32 InTextureGPUIndex,
	const FIntRect* InTextureRect) const
{
	check(IsInRenderingThread());

	if (InTexture && InTexture->IsValid() && ResourcesProxy.IsValid() && IsFrameSyncActive_RenderThread())
	{
		// Share only requested resources
		if (const FTextureShareCoreResourceRequest* ExistResourceRequest = GetData_RenderThread().FindResourceRequest(InResourceDesc))
		{
			return ShareResource_RenderThread(RHICmdList, *ExistResourceRequest, InTexture, InTextureColorDesc, InTextureGPUIndex, InTextureRect);
		}
	}

	return false;
}

DECLARE_GPU_STAT_NAMED(TextureShareObjectProxyShareResource, TEXT("TextureShare::ShareResource_RenderThread"));
bool FTextureShareObjectProxy::ShareResource_RenderThread(
	FRHICommandListImmediate& RHICmdList,
	const FTextureShareCoreResourceRequest& InResourceRequest,
	FRHITexture* InTexture,
	const FTextureShareColorDesc& InTextureColorDesc,
	const int32 InTextureGPUIndex,
	const FIntRect* InTextureRect) const
{
	if (FTextureShareResource* SharedResource = ResourcesProxy->GetSharedTexture_RenderThread(RHICmdList, CoreObject, InTexture, InResourceRequest))
	{
		UE_TS_LOG(LogTextureShareObjectProxy, Log, TEXT("%s:ShareResource_RenderThread(%s, from GPU=%d)"), *GetName_RenderThread(), *ToString(InResourceRequest), InTextureGPUIndex);

		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, TextureShareObjectProxyShareResource, "TextureShareObjectProxyShareResource");

		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShare::ShareResource_RenderThread(%s)"), *GetName_RenderThread()));

		switch (InResourceRequest.ResourceDesc.OperationType)
		{
			// Remote process request read texture, send it
		case ETextureShareTextureOp::Read:
			// Copy SrcTexture to DstSharedTextureShare immediatelly
			if (ResourcesProxy->WriteToShareTexture_RenderThread(RHICmdList, InTexture, InTextureColorDesc, InTextureRect, SharedResource))
			{
				// deferred register
				ResourcesProxy->PushRegisterResource_RenderThread(InResourceRequest, SharedResource);

				// register shared RHI resource for mgpu transfer, before sync
				ResourcesProxy->PushCrossGPUTransfer_RenderThread(ECrossGPUTransferType::BeforeSync, SharedResource, InTextureGPUIndex, InResourceRequest.GPUIndex);

				return true;
			}
			break;

			// Remote process request write texture, receive it
		case ETextureShareTextureOp::Write:
			// deferred register for receive
			ResourcesProxy->PushRegisterResource_RenderThread(InResourceRequest, SharedResource);

			// register shared RHI resource for mgpu transfer, post-sync
			ResourcesProxy->PushCrossGPUTransfer_RenderThread(ECrossGPUTransferType::AfterSync, SharedResource, InResourceRequest.GPUIndex, InTextureGPUIndex);

			// register requestes at this point and updated latter
			ResourcesProxy->PushReceiveResource_RenderThread(InResourceRequest, SharedResource, InTexture, InTextureColorDesc, InTextureRect);

			return true;

		default:
			break;
		}
	}

	UE_TS_LOG(LogTextureShareObjectProxy, Verbose, TEXT("%s:ShareResource_RenderThread('%s.%s', %s, LocalGPU=%d, RemoteGPU=%d) Skipped"), *GetName_RenderThread(),
		*InResourceRequest.ResourceDesc.ViewDesc.Id, *InResourceRequest.ResourceDesc.ResourceName, GetTEXT(InResourceRequest.ResourceDesc.OperationType), InTextureGPUIndex, InResourceRequest.GPUIndex);

	return false;
}

DECLARE_GPU_STAT_NAMED(TextureShareObjectProxyShareRenderTargetResource, TEXT("TextureShare::ShareRenderTargetResource_RenderThread"));
bool FTextureShareObjectProxy::ShareRenderTargetResource_RenderThread(
	FRHICommandListImmediate& RHICmdList,
	const FTextureShareCoreResourceRequest& InResourceRequest,
	FTextureRenderTargetResource* InRenderTargetResource,
	const FTextureShareColorDesc& InRenderTargetColorDesc,
	const int32 InRenderTargetGPUIndex,
	const FIntRect* InRenderTargetRect) const
{
	if (!InRenderTargetResource || !InRenderTargetResource->TextureRHI)
	{
		return false;
	}

	if (FTextureShareResource* SharedResource = ResourcesProxy->GetSharedTexture_RenderThread(RHICmdList, CoreObject, InRenderTargetResource->TextureRHI, InResourceRequest))
	{
		UE_TS_LOG(LogTextureShareObjectProxy, Log, TEXT("%s:ShareRenderTargetResource_RenderThread(%s, from GPU=%d)"), *GetName_RenderThread(), *ToString(InResourceRequest), InRenderTargetGPUIndex);

		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, TextureShareObjectProxyShareRenderTargetResource, "TextureShareObjectProxyShareRenderTargetResource");

		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShare::ShareRenderTargetResource_RenderThread(%s)"), *GetName_RenderThread()));

		switch (InResourceRequest.ResourceDesc.OperationType)
		{
			// Remote process request read texture, send it
		case ETextureShareTextureOp::Read:
			// Copy SrcTexture to DstSharedTextureShare immediatelly
			if (ResourcesProxy->WriteToShareTexture_RenderThread(RHICmdList, InRenderTargetResource->TextureRHI, InRenderTargetColorDesc, InRenderTargetRect, SharedResource))
			{
				// deferred register
				ResourcesProxy->PushRegisterResource_RenderThread(InResourceRequest, SharedResource);

				// register shared RHI resource for mgpu transfer, before sync
				ResourcesProxy->PushCrossGPUTransfer_RenderThread(ECrossGPUTransferType::BeforeSync, SharedResource, InRenderTargetGPUIndex, InResourceRequest.GPUIndex);

				return true;
			}
			break;

			// Remote process request write texture, receive it
		case ETextureShareTextureOp::Write:
			// deferred register for receive
			ResourcesProxy->PushRegisterResource_RenderThread(InResourceRequest, SharedResource);

			// register shared RHI resource for mgpu transfer, post-sync
			ResourcesProxy->PushCrossGPUTransfer_RenderThread(ECrossGPUTransferType::AfterSync, SharedResource, InResourceRequest.GPUIndex, InRenderTargetGPUIndex);

			// register requestes at this point and updated latter
			ResourcesProxy->PushReceiveResource_RenderThread(InResourceRequest, SharedResource, InRenderTargetResource, InRenderTargetColorDesc, InRenderTargetRect);

			return true;

		default:
			break;
		}
	}

	UE_TS_LOG(LogTextureShareObjectProxy, Verbose, TEXT("%s:ShareRenderTargetResource_RenderThread('%s.%s', %s, LocalGPU=%d, RemoteGPU=%d) Skipped"), *GetName_RenderThread(),
		*InResourceRequest.ResourceDesc.ViewDesc.Id, *InResourceRequest.ResourceDesc.ResourceName, GetTEXT(InResourceRequest.ResourceDesc.OperationType), InRenderTargetGPUIndex, InResourceRequest.GPUIndex);

	return false;
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareObjectProxy::BeginSession_RenderThread()
{
	if (!EnumHasAnyFlags(ObjectProxyFlags, ETextureShareObjectProxyFlags::SessionStarted))
	{
		UE_TS_LOG(LogTextureShareObjectProxy, Log, TEXT("%s:BeginSession_RenderThread()"), *GetName_RenderThread());

		// Force locking for the render thread. Unlocked from the game thread
		CoreObject->LockThreadMutex(ETextureShareThreadMutex::RenderingThread, true);

		ResourcesProxy = MakeUnique<FTextureShareResourcesProxy>();

		EnumAddFlags(ObjectProxyFlags, ETextureShareObjectProxyFlags::SessionStarted);

		return true;
	}

	return false;
}

bool FTextureShareObjectProxy::EndSession_RenderThread()
{
	if (EnumHasAnyFlags(ObjectProxyFlags, ETextureShareObjectProxyFlags::SessionStarted))
	{
		UE_TS_LOG(LogTextureShareObjectProxy, Log, TEXT("%s:EndSession_RenderThread()"), *GetName_RenderThread());

		EnumRemoveFlags(ObjectProxyFlags, ETextureShareObjectProxyFlags::SessionStarted);

		ResourcesProxy.Reset();

		return CoreObject->EndSession();
	}

	return false;
}

void FTextureShareObjectProxy::HandleNewFrame_RenderThread(
	const TSharedRef<FTextureShareData, ESPMode::ThreadSafe>& InTextureShareData,
	const TSharedPtr<ITextureShareContext, ESPMode::ThreadSafe>& InTextureShareContext,
	const TSharedPtr<FTextureShareSceneViewExtension, ESPMode::ThreadSafe>& InViewExtension)
{
	check(IsInRenderingThread());

	UE_TS_LOG(LogTextureShareObjectProxy, Log, TEXT("%s:HandleNewFrame_RenderThread()"), *GetName_RenderThread());

	// Assign new frame data sent from game thread
	TextureShareData = InTextureShareData;

	AssignNewContext_RenderThread(InTextureShareContext);
	AssignNewViewExtension_RenderThread(InViewExtension);

	// Mark RHI thread as dirty for flush
	if (ResourcesProxy.IsValid())
	{
		// Release unused stuff from old frame (sync lost purpose)
		ResourcesProxy->Empty();

		// Force flush RHI before any actions
		ResourcesProxy->ForceRHIFlushFlush_RenderThread();
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareObjectProxy::BeginSession_GameThread(const FTextureShareObject& In)
{
	UE_TS_LOG(LogTextureShareObjectProxy, Log, TEXT("%s:BeginSession_GameThread()"), *In.GetName());

	ENQUEUE_RENDER_COMMAND(TextureShare_BeginSession)(
		[ObjectProxyRef = In.GetObjectProxyRef()](FRHICommandListImmediate& RHICmdList)
	{
		ObjectProxyRef->BeginSession_RenderThread();
	});
}

void FTextureShareObjectProxy::EndSession_GameThread(const FTextureShareObject& In)
{
	UE_TS_LOG(LogTextureShareObjectProxy, Log, TEXT("%s:EndSession_GameThread()"), *In.GetName());

	ENQUEUE_RENDER_COMMAND(TextureShare_EndSession)(
		[ObjectProxyRef = In.GetObjectProxyRef()](FRHICommandListImmediate& RHICmdList)
	{
		ObjectProxyRef->EndSession_RenderThread();
	});
}

void FTextureShareObjectProxy::UpdateProxy_GameThread(const FTextureShareObject& In)
{
	UE_TS_LOG(LogTextureShareObjectProxy, Log, TEXT("%s:UpdateProxy_GameThread()"), *In.GetName());

	ENQUEUE_RENDER_COMMAND(TextureShare_UpdateObjectProxy)(
		[
			ObjectProxyRef = In.GetObjectProxyRef(),
			NewTextureShareData = In.TextureShareData,
			NewTextureShareContext = In.TextureShareContext,
			SceneViewExtension = In.ViewExtension
		](FRHICommandListImmediate& RHICmdList)
	{
		ObjectProxyRef->HandleNewFrame_RenderThread(NewTextureShareData, NewTextureShareContext, SceneViewExtension);
	});
}

void FTextureShareObjectProxy::OnTextureSharePreBeginFrameSync_GameThread(const FTextureShareObject& In)
{
	ENQUEUE_RENDER_COMMAND(TextureShare_OnBeginFrameSyncEvent)(
		[
			ObjectProxyRef = In.GetObjectProxyRef(),
			NewTextureShareContext = In.TextureShareContext
		](FRHICommandListImmediate& RHICmdList)
	{
		// Always assign new context right before the new frame
		ObjectProxyRef->AssignNewContext_RenderThread(NewTextureShareContext);

		if (ITextureShareCallbacks::Get().OnTextureSharePreBeginFrameSync_RenderThread().IsBound())
		{
			ITextureShareCallbacks::Get().OnTextureSharePreBeginFrameSync_RenderThread().Broadcast(RHICmdList, *ObjectProxyRef);
		}

	});
}