// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ITextureShareObjectProxy.h"
#include "ITextureShareCoreObject.h"
#include "Containers/TextureShareContainers.h"
#include "Templates/SharedPointer.h"

class FTextureShareResourcesProxy;
class FTextureShareSceneViewExtension;
class FTextureShareObject;

/**
 * TextureShare proxy object
 */
class FTextureShareObjectProxy
	: public ITextureShareObjectProxy
	, public TSharedFromThis<FTextureShareObjectProxy, ESPMode::ThreadSafe>
{
	friend class FTextureShareObject;

public:
	FTextureShareObjectProxy(const TSharedRef<ITextureShareCoreObject, ESPMode::ThreadSafe>& InCoreObject);
	virtual ~FTextureShareObjectProxy();

public:
	// ITextureShareObjectProxy
	virtual const ITextureShareContext* GetTextureShareContext_RenderThread() const override;

	virtual const FString& GetName_RenderThread() const override;
	virtual FTextureShareCoreObjectDesc GetObjectDesc_RenderThread() const override;

	virtual bool IsActive_RenderThread() const override;
	virtual bool IsFrameSyncActive_RenderThread() const override;

	virtual bool BeginFrameSync_RenderThread(FRHICommandListImmediate& RHICmdList) const override;
	virtual bool FrameSync_RenderThread(FRHICommandListImmediate& RHICmdList, const ETextureShareSyncStep InSyncStep) const override;
	virtual bool EndFrameSync_RenderThread(FRHICommandListImmediate& RHICmdList) const override;

	virtual FTextureShareCoreProxyData& GetCoreProxyData_RenderThread() override;
	virtual const FTextureShareCoreProxyData& GetCoreProxyData_RenderThread() const override;

	virtual TArray<FTextureShareCoreObjectProxyData> GetReceivedCoreObjectProxyData_RenderThread() const override;

	virtual const FTextureShareData& GetData_RenderThread() const override;

	virtual const TSharedPtr<FTextureShareSceneViewExtension, ESPMode::ThreadSafe>& GetViewExtension_RenderThread() const override
	{
		return ViewExtension;
	}

	virtual ETextureShareObjectProxyFlags GetObjectProxyFlags() const override;
	virtual void SetObjectProxyFlags(const ETextureShareObjectProxyFlags InFlags) override;

	virtual bool ShareResource_RenderThread(
		FRHICommandListImmediate& RHICmdList,
		const FTextureShareCoreResourceDesc& InResourceDesc,
		FRHITexture* InTexture,
		const FTextureShareColorDesc& InTextureColorDesc,
		const int32 InTextureGPUIndex,
		const FIntRect* InTextureRect = nullptr) const override;

	virtual bool ShareResource_RenderThread(
		FRDGBuilder& GraphBuilder,
		const FTextureShareCoreResourceDesc& InResourceDesc,
		const FRDGTextureRef& InTextureRef,
		const FTextureShareColorDesc& InTextureColorDesc,
		const int32 InTextureGPUIndex,
		const FIntRect* InTextureRect = nullptr) const override;

	virtual bool ShareResource_RenderThread(
		FRHICommandListImmediate& RHICmdList,
		const FTextureShareCoreResourceRequest& InResourceRequest,
		FRHITexture* InTexture,
		const FTextureShareColorDesc& InTextureColorDesc,
		const int32 InTextureGPUIndex,
		const FIntRect* InTextureRect = nullptr) const override;

	virtual bool ShareResource_RenderThread(
		FRDGBuilder& GraphBuilder,
		const FTextureShareCoreResourceRequest& InResourceRequest,
		const FRDGTextureRef& InTextureRef,
		const FTextureShareColorDesc& InTextureColorDesc,
		const int32 InTextureGPUIndex,
		const FIntRect* InTextureRect = nullptr) const override;

	virtual bool ShareRenderTargetResource_RenderThread(
		FRHICommandListImmediate& RHICmdList,
		const FTextureShareCoreResourceRequest& InResourceRequest,
		FTextureRenderTargetResource* InRenderTargetResource,
		const FTextureShareColorDesc& InRenderTargetColorDesc,
		const int32 InRenderTargetGPUIndex,
		const FIntRect* InRenderTargetRect = nullptr) const override;
	//~ITextureShareObjectProxy


protected:
	bool BeginSession_RenderThread();
	bool EndSession_RenderThread();

	/** Set the data from the game thread to the proxy object.*/
	void HandleNewFrame_RenderThread(
		const TSharedRef<FTextureShareData, ESPMode::ThreadSafe>& InTextureShareData,
		const TSharedPtr<ITextureShareContext, ESPMode::ThreadSafe>& InTextureShareContext,
		const TSharedPtr<FTextureShareSceneViewExtension, ESPMode::ThreadSafe>& InViewExtension);

	bool DoFrameSync_RenderThread(FRHICommandListImmediate& RHICmdList, const ETextureShareSyncStep InSyncStep) const;

	/** This function assigns a new context to the proxy object and handles multithreading issues. */
	void AssignNewContext_RenderThread(const TSharedPtr<ITextureShareContext, ESPMode::ThreadSafe>& InTextureShareContext);

	/** This function assigns a new ViewExtension to the proxy object and handles multithreading issues. */
	void AssignNewViewExtension_RenderThread(const TSharedPtr<FTextureShareSceneViewExtension, ESPMode::ThreadSafe>& InViewExtension);

	/** The CoreObject should be destroyed in the game thread. */
	void ReleaseCoreObject_RenderThread();

	static void BeginSession_GameThread(const FTextureShareObject& In);
	static void EndSession_GameThread(const FTextureShareObject& In);
	static void UpdateProxy_GameThread(const FTextureShareObject& In);


	/** Called from the FTextureShareObject::BeginFrameSync(). */
	static void OnTextureSharePreBeginFrameSync_GameThread(const FTextureShareObject& In);

private:
	// TS Core lib object
	const TSharedRef<ITextureShareCoreObject, ESPMode::ThreadSafe> CoreObject;

	// Object data from game thread
	TSharedRef<FTextureShareData, ESPMode::ThreadSafe> TextureShareData;

	// Is an abstract container that can be used by the user to handle callback logic.
	TSharedPtr<ITextureShareContext, ESPMode::ThreadSafe> TextureShareContext;

	// Extra flags
	mutable ETextureShareObjectProxyFlags ObjectProxyFlags = ETextureShareObjectProxyFlags::None;

	// All RHI resources and interfaces
	TUniquePtr<FTextureShareResourcesProxy> ResourcesProxy;

	// Scene view extension
	TSharedPtr<FTextureShareSceneViewExtension, ESPMode::ThreadSafe> ViewExtension;
};
