// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "TextureShareBlueprintContainersBase.h"
#include "TextureShareBlueprintContainers.generated.h"

/**
 * TextureShare UObject container
 */
UCLASS(Blueprintable)
class TEXTURESHARE_API UTextureShareObject
	: public UObject
{
	GENERATED_BODY()

public:
	UTextureShareObject();
	virtual ~UTextureShareObject();

public:
	// Override CustomData SendParameters
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Send Custom Data"), Category = "TextureShare")
	void SendCustomData(const TMap<FString, FString>& InSendParameters);

public:
	// Enable this texture share object
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	bool bEnable = true;

	// Object description
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	FTextureShareObjectDesc Desc;

	// Shared resources
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	FTextureShareTexturesDesc Textures;

	// Shared custom data
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	FTextureShareCustomData CustomData;
};

/**
 * TextureShare UObject interface
 */
UCLASS(Blueprintable)
class TEXTURESHARE_API UTextureShare
	: public UObject
{
	GENERATED_BODY()

public:
	UTextureShare();
	virtual ~UTextureShare();

public:
	bool IsEnabled() const
	{
		return bEnable && TextureShareObjects.Num() > 0;
	}

	// return existing textureshare object names (duplicates are ignored)
	TSet<FString> GetTextureShareObjectNames() const;
	
	// Return enabled UObject by name
	UTextureShareObject* GetTextureShareObject(const FString& InShareName) const;

public:
	// Create new or get exist UTextureShare object
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get or Create TextureShare Object"), Category = "TextureShare")
	UTextureShareObject* GetOrCreateTextureShareObject(const FString& InShareName);

	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Remove TextureShare Object"), Category = "TextureShare")
	bool RemoveTextureShareObject(const FString& InShareName);

	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Get All TextureShare Objects"), Category = "TextureShare")
	const TArray<UTextureShareObject*> GetTextureShareObjects() const;

public:
	// Enable sharing for all objects
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	bool bEnable = true;

	// Unique process name (optional). When empty, used default name
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	FString ProcessName;

	// Objects for sharing
	UPROPERTY(Transient)
	TArray<TObjectPtr<UTextureShareObject>> TextureShareObjects;
};
