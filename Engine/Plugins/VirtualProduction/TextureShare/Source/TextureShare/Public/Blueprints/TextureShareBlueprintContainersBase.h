// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Engine/TextureRenderTarget.h"

#include "TextureShareBlueprintContainersBase.generated.h"

/**
 * Texture with name for sending
 */
USTRUCT(Blueprintable)
struct TEXTURESHARE_API FTextureShareSendTextureDesc
{
	GENERATED_BODY()

public:
	// Resource name (used for IPC)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	FString Name;

	// Resource to send
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	TObjectPtr<UTexture> Texture;
};

/**
 * Texture with name for receive
 */
USTRUCT(Blueprintable)
struct TEXTURESHARE_API FTextureShareReceiveTextureDesc
{
	GENERATED_BODY()

public:
	// Resource name (used for IPC)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	FString Name;

	// Resource to receive
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	TObjectPtr<UTextureRenderTarget> Texture;
};

/**
 * Custom textures for sharing
 */
USTRUCT(Blueprintable)
struct TEXTURESHARE_API FTextureShareTexturesDesc
{
	GENERATED_BODY()

public:
	// Send all this textures
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	TArray<FTextureShareSendTextureDesc> SendTextures;

	// Receive all this textures
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	TArray<FTextureShareReceiveTextureDesc> ReceiveTextures;
};

/**
 * Custom data for sharing
 */
USTRUCT(Blueprintable)
struct TEXTURESHARE_API FTextureShareCustomData
{
	GENERATED_BODY()

public:
	// These custom data will be sent to remote processes.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	TMap<FString, FString> SendParameters;

	// This user data is received from remote processes. Updated every frame tick
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	TMap<FString, FString> ReceivedParameters;
};

/**
 * Sync settings
 */
USTRUCT(Blueprintable)
struct TEXTURESHARE_API FTextureShareObjectSyncSettings
{
	GENERATED_BODY()

public:
	// MaxMillisecondsToWait for sync. 0 -infinite
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	int FrameSyncTimeOut = 1000;

	// MaxMillisecondsToWait for first connection. 0 -infinite
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	int FrameConnectTimeOut = 5000;
};

/**
 * Object descriptor
 */
USTRUCT(Blueprintable)
struct TEXTURESHARE_API FTextureShareObjectDesc
{
	GENERATED_BODY()

public:
	FString GetTextureShareObjectName() const;

public:
	// Unique share name (case insensitive). When empty, used default name
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	FString ShareName;

	// Settings
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TextureShare")
	FTextureShareObjectSyncSettings Settings;
};
