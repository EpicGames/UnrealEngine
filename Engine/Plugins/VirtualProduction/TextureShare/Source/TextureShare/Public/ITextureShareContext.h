// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"

/**
 * TextureShare context
 * Is an abstract container that should be used by the user to handle callback logic.
 * 
 * Multithreading:
 * The implementation of TS on the UE side is multithreaded.
 * 
 * Every frame in the game thread we have to create a new context and populate it with new data.
 * Then set it using the ITextureShareObject::SetTextureShareContext() function. 
 * 
 * This context is passed from the game thread to the rendering thread.
 * (from the ITextureShareObject to the ITextureShareObjectProxy)
 * With this approach, we get unique context data for both threads.
 *
 * Callbacks:
 * Global TS callbacks are implemented in ITextureShareCallbacks.
 * All of them can be used by different implementations at the same time.
 * Therefore, we must separate the logic of these callbacks from each other.
 * This can easily be done by implementing them in unique classes that are children of ITextureShareContext.
 * For example, by checking the context of a TS object using one of these two functions:
 *   if(ITextureShareObject::GetTextureShareContext() == this) ...
 *   if(ITextureShareObjectProxy::GetTextureShareContext_RenderThread() == this) ...
 * 
 * Custom implementation:
 * Each custom implementation can create and use a new context class based on the ITextureShareContext class.
 * It may also contain additional custom data for callbacks logic.
 */
class TEXTURESHARE_API ITextureShareContext
	: public TSharedFromThis<ITextureShareContext, ESPMode::ThreadSafe>
{
public:
	virtual ~ITextureShareContext()
	{
		check(IsInGameThread());

		UnregisterTextureShareContextCallbacks();
	};

	/** Register callbacks for the game thread. */
	virtual void RegisterTextureShareContextCallbacks() {};

	/** Unregister callbacks for the game thread. */
	virtual void UnregisterTextureShareContextCallbacks() {};

	/** Register callbacks for the rendering thread. */
	virtual void RegisterTextureShareContextCallbacks_RenderThread() {};

	/** Unregister callbacks for the rendering thread. */
	virtual void UnregisterTextureShareContextCallbacks_RenderThread() {};

	/** A quick and dirty way to determine which TS data (sub)class this is. */
	virtual FName GetRTTI() const { return NAME_None; }

	/** Returns true if the given object is of the same type. */
	bool IsA(const ITextureShareContext&& Other) const { return GetRTTI() == Other.GetRTTI(); }
};
