// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

[SupportedPlatforms("Win64")]
public class TextureShare : ModuleRules
{
	public TextureShare(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PrivateIncludePaths.AddRange(new string[] {
			Path.Combine(GetModuleDirectory("Renderer"), "Private"),
			Path.Combine(GetModuleDirectory("TextureShareCore"), "Private"),
		});

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"Projects"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine",
				"RHI",
				"RHICore",
				"Renderer",
				"RenderCore",
				"Slate",
				"SlateCore",
				"TextureShareCore"
			}
		);

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"D3D11RHI",
					"D3D12RHI"
				}
			);
		}

		// Show more log
		bool bEnableExtraDebugLog = false;
		if (bEnableExtraDebugLog)
		{
			//Show log in SDK-for Debug and DebugGame builds
			PublicDefinitions.Add("TEXTURESHARE_DEBUGLOG=1");
		}
		else
		{
			PublicDefinitions.Add("TEXTURESHARE_DEBUGLOG=0");
		}

		// Allow using Vulkan render device
		bool bSupportDeviceVulkan = false;

		if (bSupportDeviceVulkan)
		{
			PublicDefinitions.Add("TEXTURESHARE_VULKAN=1");

			// Support Vulkan device
			PrivateDependencyModuleNames.AddRange(new string[] { "VulkanRHI" });
			PrivateIncludePathModuleNames.Add("VulkanRHI");

			AddEngineThirdPartyPrivateStaticDependencies(Target, "Vulkan");
		}
		else
		{
			PublicDefinitions.Add("TEXTURESHARE_VULKAN=0");
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX12");
		}

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("UnrealEd");
		}
	}
}
