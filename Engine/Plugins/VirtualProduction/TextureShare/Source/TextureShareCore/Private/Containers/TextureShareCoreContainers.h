// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/TextureShareCoreContainers_Data.h"
#include "Containers/TextureShareCoreContainers_ProxyData.h"
#include "Containers/TextureShareCoreContainers_SyncSettings.h"

/**
 * TextureShareCore object data container
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreObjectData
	: public ITextureShareSerialize
{
	// Object information
	FTextureShareCoreObjectDesc Desc;

	// Setup frame data on game thread before rendering call
	FTextureShareCoreData Data;

public:
	virtual ~FTextureShareCoreObjectData() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream& Stream) override
	{
		return Stream << Desc << Data;
	}
};

/**
 * TextureShareCore object proxy data container
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreObjectProxyData
	: public ITextureShareSerialize
{
	// Object ID
	FTextureShareCoreObjectDesc Desc;

	// Setup frame data on game thread before rendering call
	FTextureShareCoreProxyData ProxyData;

public:
	virtual ~FTextureShareCoreObjectProxyData() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << Desc << ProxyData;
	}
};
