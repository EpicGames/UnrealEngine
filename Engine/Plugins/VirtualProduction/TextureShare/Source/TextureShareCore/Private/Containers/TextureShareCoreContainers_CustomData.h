// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Serialize/TextureShareCoreSerialize.h"

/**
 * Custom data pair: Key + Value
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreCustomData
	: public ITextureShareSerialize
{
	// Custom data key
	FString Key;

	// Custom data value
	FString Value;

public:
	virtual ~FTextureShareCoreCustomData() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << Key << Value;
	}

public:
	FTextureShareCoreCustomData() = default;

	FTextureShareCoreCustomData(const wchar_t* InKey, const wchar_t* InValue)
		: Key(InKey), Value(InValue)
	{}

	FTextureShareCoreCustomData(const FString& InKey, const FString& InValue)
		: Key(InKey), Value(InValue)
	{}

	bool EqualsFunc(const wchar_t* InKey) const
	{
		return Key == InKey;
	}

	bool EqualsFunc(const FString& InKey) const
	{
		return Key == InKey;
	}

	bool EqualsFunc(const FTextureShareCoreCustomData& InParameter) const
	{
		return Key == InParameter.Key;
	}

	bool operator==(const FTextureShareCoreCustomData& InParameter) const
	{
		return Key == InParameter.Key;
	}
};
