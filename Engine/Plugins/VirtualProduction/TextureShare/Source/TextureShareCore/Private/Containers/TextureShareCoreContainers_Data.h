// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/TextureShareCoreContainers_FrameMarker.h"
#include "Containers/TextureShareCoreContainers_ViewDesc.h"
#include "Containers/TextureShareCoreContainers_ResourceRequest.h"
#include "Containers/TextureShareCoreContainers_ManualProjection.h"
#include "Containers/TextureShareCoreContainers_CustomData.h"

/**
 * Game data of the TextureShareCore object
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreData
	: public ITextureShareSerialize
{
	// Current frame marker
	FTextureShareCoreFrameMarker FrameMarker;

	// Supported views of this object
	TArraySerializable<FTextureShareCoreViewDesc> SupportedViews;

	// Resource requests from this object to remote processes
	TArraySerializable<FTextureShareCoreResourceRequest> ResourceRequests;

	// Manual projection requests from this object to remote processes
	TArraySerializable<FTextureShareCoreManualProjection> ManualProjections;

	// Sources of used manual projections in local process
	TArraySerializable<FTextureShareCoreManualProjectionSource> ManualProjectionsSources;

	// Custom abstract data of this object
	TArraySerializable<FTextureShareCoreCustomData> CustomData;

public:
	virtual ~FTextureShareCoreData() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << FrameMarker << SupportedViews << ResourceRequests << ManualProjections << ManualProjectionsSources << CustomData;
	}

public:
	void ResetData()
	{
		SupportedViews.Empty();
		ResourceRequests.Empty();
		ManualProjections.Empty();
		ManualProjectionsSources.Empty();
		CustomData.Empty();
	}
};
