// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct ID3D11Device;
struct ID3D11Texture2D;
