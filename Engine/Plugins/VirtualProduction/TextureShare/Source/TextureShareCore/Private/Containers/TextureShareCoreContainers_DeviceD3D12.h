// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct ID3D12Device;
struct ID3D12Resource;

struct ID3D12GraphicsCommandList;
struct ID3D12DescriptorHeap;

#include <DXGIFormat.h>
