// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Serialize/TextureShareCoreSerialize.h"

typedef struct VkInstance_T*       VkInstance;
typedef struct VkPhysicalDevice_T* VkPhysicalDevice;
typedef struct VkDevice_T*         VkDevice;
typedef struct VkDeviceMemory_T*   VkDeviceMemory;
typedef struct VkImage_T*          VkImage;

/**
 * Vulkan device context
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareDeviceVulkanContext
	: public ITextureShareSerialize
{
	FTextureShareDeviceVulkanContext() = default;
	FTextureShareDeviceVulkanContext(VkInstance InInstance, VkPhysicalDevice InPhysicalDevice, VkDevice InDevice)
	{
		Instance = (Windows::HANDLE)(InInstance);
		PhysicalDevice = (Windows::HANDLE)(InPhysicalDevice);
		Device = (Windows::HANDLE)(InDevice);
	}

	bool IsValid() const
	{
		return Instance != nullptr && Device!= nullptr && PhysicalDevice!=nullptr;
	}

	VkInstance GetInstance() const
	{
		return (VkInstance)(Instance);
	}

	VkPhysicalDevice GetPhysicalDevice() const
	{
		return (VkPhysicalDevice)(PhysicalDevice);
	}

	VkDevice GetDevice() const
	{
		return (VkDevice)(Device);
	}

public:
	virtual ~FTextureShareDeviceVulkanContext() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << Instance << PhysicalDevice << Device;
	}

private:
	Windows::HANDLE Instance = nullptr;
	Windows::HANDLE PhysicalDevice = nullptr;
	Windows::HANDLE Device = nullptr;
};

/**
 * Vulkan resource context
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareDeviceVulkanResource
	: public ITextureShareSerialize
{
	FTextureShareDeviceVulkanResource() = default;

	FTextureShareDeviceVulkanResource(VkDeviceMemory InAllocationHandle, const uint64 InAllocationOffset = 0)
	{
		AllocationHandle = (Windows::HANDLE)(InAllocationHandle);
		AllocationOffset = InAllocationOffset;
	}

	void operator==(const FTextureShareDeviceVulkanResource& In)
	{
		AllocationHandle = In.AllocationHandle;
		AllocationOffset = In.AllocationOffset;
	}

	bool IsValid() const
	{
		return AllocationHandle != nullptr;
	}

	void* GetNativeResourcePtr() const
	{
		return AllocationHandle;
	}

	VkDeviceMemory GetAllocationHandle() const
	{
		return (VkDeviceMemory)(AllocationHandle);
	}

	uint64 GetAllocationOffset() const
	{
		return AllocationOffset;
	}

public:
	virtual ~FTextureShareDeviceVulkanResource() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << AllocationHandle << AllocationOffset;
	}

private:
	Windows::HANDLE AllocationHandle = nullptr;
	uint64          AllocationOffset = 0;
};
