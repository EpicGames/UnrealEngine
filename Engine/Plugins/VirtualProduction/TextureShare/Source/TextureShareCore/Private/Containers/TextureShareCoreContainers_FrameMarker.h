// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/TextureShareCoreContainers_ObjectDesc.h"

/**
 * Frame marker. Sync purpose
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreFrameMarker
	: public ITextureShareSerialize
{
	uint32 CustomFrameIndex1 = 0;
	uint64 CustomFrameIndex2 = 0;

public:
	virtual ~FTextureShareCoreFrameMarker() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << CustomFrameIndex1 << CustomFrameIndex2;
	}

public:
	void NextFrame()
	{
		// Simple frame marker incremental update
		CustomFrameIndex1++;
		CustomFrameIndex2++;
	}
};

/**
 * Frame marker paired with Object descriptor. Sync purpose
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreObjectFrameMarker
	: public ITextureShareSerialize
{
	// Projection data source process Object ID
	FTextureShareCoreObjectDesc ObjectDesc;

	// Projection data source process frame marker
	FTextureShareCoreFrameMarker FrameMarker;

public:
	virtual ~FTextureShareCoreObjectFrameMarker() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << ObjectDesc << FrameMarker;
	}

public:
	FTextureShareCoreObjectFrameMarker() = default;

	FTextureShareCoreObjectFrameMarker(const FTextureShareCoreObjectDesc& InObjectDesc, const FTextureShareCoreFrameMarker& InFrameMarker)
		: ObjectDesc(InObjectDesc), FrameMarker(InFrameMarker)
	{ }

	bool EqualsFunc(const FTextureShareCoreObjectDesc& InObjectDesc) const
	{
		return InObjectDesc.EqualsFunc(ObjectDesc);
	}

	bool EqualsFunc(const FTextureShareCoreObjectFrameMarker& InObjectFrameMarker) const
	{
		return InObjectFrameMarker.ObjectDesc.EqualsFunc(ObjectDesc);
	}

	bool operator==(const FTextureShareCoreObjectFrameMarker& InObjectFrameMarker) const
	{
		return InObjectFrameMarker.ObjectDesc == ObjectDesc;
	}
};
