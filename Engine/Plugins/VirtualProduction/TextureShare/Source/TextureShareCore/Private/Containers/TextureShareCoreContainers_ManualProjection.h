// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/TextureShareCoreContainers_FrameMarker.h"
#include "Containers/TextureShareCoreContainers_ViewDesc.h"

/**
 * Manual projection settings for specified view
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreManualProjection
	: public ITextureShareSerialize
{
	// Apply manual projection for this view
	FTextureShareCoreViewDesc ViewDesc;

	ETextureShareCoreSceneViewManualProjectionType ProjectionType = ETextureShareCoreSceneViewManualProjectionType::FrustumAngles;

	ETextureShareViewRotationDataType   ViewRotationType = ETextureShareViewRotationDataType::Original;
	ETextureShareViewLocationDataType   ViewLocationType = ETextureShareViewLocationDataType::Original;

	// Projection matrix
	FMatrix ProjectionMatrix = FMatrix::Identity;

	// View rotation
	FRotator ViewRotation = FRotator::ZeroRotator;

	// View location
	FVector ViewLocation = FVector::ZeroVector;

	// Frustum angles
	struct FFrustumAngles
	{
		float Left = -30.f;
		float Right = 30.f;
		float Top = 30.f;
		float Bottom = -30.f;
	} FrustumAngles;

public:
	virtual ~FTextureShareCoreManualProjection() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << ViewDesc << ProjectionType << ViewRotationType << ViewLocationType
			<< ProjectionMatrix << ViewRotation << ViewLocation
			<< FrustumAngles.Left << FrustumAngles.Right << FrustumAngles.Top << FrustumAngles.Bottom;
	}

public:
	bool EqualsFunc(const FString& InViewId) const
	{
		return ViewDesc.EqualsFunc(InViewId);
	}

	bool EqualsFunc(const FTextureShareCoreViewDesc& InViewDesc) const
	{
		return ViewDesc.EqualsFunc(InViewDesc);
	}

	bool EqualsFunc(const FTextureShareCoreManualProjection& InManualProjection) const
	{
		return ViewDesc.EqualsFunc(InManualProjection.ViewDesc);
	}
};

/**
 * Manual projection source info
 * When using a manual projection from a remote process, store the description of the remote object with a frame marker
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreManualProjectionSource
	: public ITextureShareSerialize
{
	// View desc type of this resource (support stereo)
	FTextureShareCoreViewDesc ViewDesc;

	// remote process object and framemarker info
	FTextureShareCoreObjectFrameMarker ObjectFrameMarker;

public:
	virtual ~FTextureShareCoreManualProjectionSource() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << ViewDesc << ObjectFrameMarker;
	}

public:
	FTextureShareCoreManualProjectionSource() = default;

	FTextureShareCoreManualProjectionSource(const FTextureShareCoreViewDesc& InViewDesc, const FTextureShareCoreObjectFrameMarker& InObjectFrameMarker)
		: ViewDesc(InViewDesc), ObjectFrameMarker(InObjectFrameMarker)
	{ }

	bool EqualsFunc(const FTextureShareCoreViewDesc& InViewDesc) const
	{
		return ViewDesc.EqualsFunc(InViewDesc);
	}

	bool EqualsFunc(const FTextureShareCoreManualProjectionSource& InManualProjectionSource) const
	{
		return ViewDesc.EqualsFunc(InManualProjectionSource.ViewDesc);
	}

	bool operator==(const FTextureShareCoreManualProjectionSource& InManualProjectionSource) const
	{
		return ViewDesc == InManualProjectionSource.ViewDesc;
	}
};
