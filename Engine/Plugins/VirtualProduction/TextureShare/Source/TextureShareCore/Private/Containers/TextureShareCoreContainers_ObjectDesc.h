// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Serialize/TextureShareCoreSerialize.h"
#include "Containers/TextureShareCoreContainers_SyncSettings.h"

/**
 * Process descriptor
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreObjectProcessDesc
	: public ITextureShareSerialize
{
	// Process name (user-defined)
	FString ProcessId;

	// Unique process guid
	FGuid ProcessGuid;

	// The local process type
	ETextureShareProcessType ProcessType = ETextureShareProcessType::Undefined;

	// Local process device type
	ETextureShareDeviceType DeviceType = ETextureShareDeviceType::Undefined;

public:
	virtual ~FTextureShareCoreObjectProcessDesc() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << ProcessId << ProcessGuid << ProcessType << DeviceType;
	}
};

/**
 * Object sync logic state
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreObjectSyncState
	: public ITextureShareSerialize
{
	ETextureShareSyncStep  Step = ETextureShareSyncStep::Undefined;
	ETextureShareSyncState State = ETextureShareSyncState::Undefined;

	ETextureShareSyncStep NextStep = ETextureShareSyncStep::Undefined;
	ETextureShareSyncStep PrevStep = ETextureShareSyncStep::Undefined;

public:
	virtual ~FTextureShareCoreObjectSyncState() = default;
	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream& Stream) override
	{
		return Stream << Step << State << NextStep << PrevStep;
	}
};

/**
 * Object sync info
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreObjectSync
	: public ITextureShareSerialize
{
	// Object sync logic state
	FTextureShareCoreObjectSyncState SyncState;

	// Bit-storage for sync steps used by this process
	// 64 bits. Max steps=64
	uint64 SyncStepSettings = 0;

	// Last access time from this object
	uint64 LastAccessTime = 0;

public:
	virtual ~FTextureShareCoreObjectSync() = default;
	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream& Stream) override
	{
		return Stream << SyncState << SyncStepSettings << LastAccessTime;
	}

public:
	bool IsStepEnabled(const ETextureShareSyncStep InStep) const
	{
		if (InStep == ETextureShareSyncStep::InterprocessConnection)
		{
			return true;
		}

		const uint8 BitIndex = (uint8)InStep;

		if (BitIndex >= 0 && BitIndex < 64)
		{
			return (SyncStepSettings & (1ULL << BitIndex)) != 0;
		}

		return false;
	}

	void SetSyncStepSettings(const FTextureShareCoreSyncSettings& InSyncSettings)
	{
		// Update bit-storage for used steps
		SyncStepSettings = 0;
		for (const ETextureShareSyncStep& StepIt : InSyncSettings.FrameSyncSettings.Steps)
		{
			const uint8 BitIndex = (uint8)StepIt;
			if (BitIndex >= 0 && BitIndex < 64)
			{
				SyncStepSettings |= (1ULL << BitIndex);
			}
		}
	}
};

/**
 * Object unique description
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreObjectDesc
	: public ITextureShareSerialize
{
	// Texture share name (TS objects with the same ShareName are trying to connect)
	FString ShareName;

	// Unique object guid. Generated unique for a local process
	FGuid ObjectGuid;

	// Object Sync info
	FTextureShareCoreObjectSync Sync;

	// Local process description
	FTextureShareCoreObjectProcessDesc ProcessDesc;

public:
	virtual ~FTextureShareCoreObjectDesc() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << ShareName << ObjectGuid << Sync << ProcessDesc;
	}

public:
	FTextureShareCoreObjectDesc() = default;

	bool EqualsFunc(const FGuid& InObjectGuid) const
	{
		return ObjectGuid == InObjectGuid;
	}

	bool EqualsFunc(const FTextureShareCoreObjectDesc& InObjectDesc) const
	{
		return ObjectGuid == InObjectDesc.ObjectGuid;
	}

	bool operator==(const FTextureShareCoreObjectDesc& InObjectDesc) const
	{
		return ObjectGuid == InObjectDesc.ObjectGuid;
	}
};
