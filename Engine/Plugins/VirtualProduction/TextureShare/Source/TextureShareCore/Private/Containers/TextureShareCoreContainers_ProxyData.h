// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/TextureShareCoreContainers_FrameMarker.h"
#include "Containers/TextureShareCoreContainers_SceneData.h"
#include "Containers/TextureShareCoreContainers_ResourceHandle.h"

/**
 * Proxy data of the TextureShareCore object
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreProxyData
	: public ITextureShareSerialize
{
	// Frame markers on local process (Copied from game thread value)
	// Must be copied from the game thread object data immediately after BeginFrameSync_RenderThread.
	// **Currently, only the UE process side makes changes to this property.**
	FTextureShareCoreFrameMarker FrameMarker;

	// Frame markers of all connected objects from game thread
	// Must be copied from the game thread object data immediately after BeginFrameSync_RenderThread.
	// **Currently, only the UE process side makes changes to this property.**
	TArraySerializable<FTextureShareCoreObjectFrameMarker> RemoteFrameMarkers;

	// Scene data (viewports, matrices, etc)
	// **Currently, only the UE process side makes changes to this property.**
	TArraySerializable<FTextureShareCoreSceneViewData> SceneData;

	// Descriptors for shared resources. Change only from the owner of the resource.
	// **Currently, only the UE process side makes changes to this property.**
	TArraySerializable<FTextureShareCoreResourceHandle> ResourceHandles;

public:
	virtual ~FTextureShareCoreProxyData() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << FrameMarker << RemoteFrameMarkers << SceneData << ResourceHandles;
	}

public:
	void ResetProxyData()
	{
		RemoteFrameMarkers.Empty();
		SceneData.Empty();
		ResourceHandles.Empty();
	}
};
