// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/TextureShareCoreContainers_ViewDesc.h"

/**
 * Resource descriptor
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreResourceDesc
	: public ITextureShareSerialize
{
	// Resource name
	FString ResourceName;

	// Resource owner
	FTextureShareCoreViewDesc ViewDesc;

	// Texture operation type (read or write)
	ETextureShareTextureOp OperationType = ETextureShareTextureOp::Read;

	// resource sync order for logic (default value '-1'unordered)
	ETextureShareSyncStep SyncStep = ETextureShareSyncStep::Undefined;

	// Resource type
	ETextureShareResourceType ResourceType = ETextureShareResourceType::Default;

public:
	virtual ~FTextureShareCoreResourceDesc() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << ResourceName << ViewDesc << OperationType << SyncStep << ResourceType;
	}

public:
	FTextureShareCoreResourceDesc(const FString& InResourceName, const FTextureShareCoreViewDesc& InViewDesc, const ETextureShareTextureOp InOperationType, const ETextureShareSyncStep InSyncStep, const ETextureShareResourceType& InResourceType)
		: ResourceName(InResourceName), ViewDesc(InViewDesc), OperationType(InOperationType), SyncStep(InSyncStep), ResourceType(InResourceType)
	{ }

	FTextureShareCoreResourceDesc(const FString& InResourceName, const FTextureShareCoreViewDesc& InViewDesc, const ETextureShareTextureOp InOperationType, const ETextureShareSyncStep InSyncStep)
		: ResourceName(InResourceName), ViewDesc(InViewDesc), OperationType(InOperationType), SyncStep(InSyncStep)
	{ }

	FTextureShareCoreResourceDesc(const FString& InResourceName, const FTextureShareCoreViewDesc& InViewDesc, const ETextureShareTextureOp InOperationType)
		: ResourceName(InResourceName), ViewDesc(InViewDesc), OperationType(InOperationType)
	{ }

	FTextureShareCoreResourceDesc(const FString& InResourceName, const ETextureShareTextureOp InOperationType, const ETextureShareSyncStep InSyncStep)
		: ResourceName(InResourceName), OperationType(InOperationType), SyncStep(InSyncStep)
	{ }

	FTextureShareCoreResourceDesc(const FString& InResourceName, const ETextureShareTextureOp InOperationType)
		: ResourceName(InResourceName), OperationType(InOperationType)
	{ }

	FTextureShareCoreResourceDesc(const FString& InResourceName)
		: ResourceName(InResourceName)
	{ }

	FTextureShareCoreResourceDesc() = default;

public:
	bool EqualsFunc(const FTextureShareCoreViewDesc& InViewDesc) const
	{
		return ViewDesc.EqualsFunc(InViewDesc);
	}

	bool EqualsFunc(const FTextureShareCoreResourceDesc& InResourceDesc) const
	{
		return ViewDesc.EqualsFunc(InResourceDesc.ViewDesc)
			&& ResourceName == InResourceDesc.ResourceName
			&& ResourceType == InResourceDesc.ResourceType
			&& (OperationType == InResourceDesc.OperationType
				|| OperationType == ETextureShareTextureOp::Undefined
				|| InResourceDesc.OperationType == ETextureShareTextureOp::Undefined);
	}

	bool operator==(const FTextureShareCoreResourceDesc& InResourceDesc) const
	{
		return ViewDesc == InResourceDesc.ViewDesc
			&& ResourceName == InResourceDesc.ResourceName
			&& ResourceType == InResourceDesc.ResourceType
			&& OperationType == InResourceDesc.OperationType;
	}
};
