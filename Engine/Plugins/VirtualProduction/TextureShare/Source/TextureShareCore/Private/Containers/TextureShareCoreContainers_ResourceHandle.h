// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/TextureShareCoreContainers_ResourceDesc.h"

/**
 * Resource handle data (Shared resource handlers and other data)
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreResourceHandle
	: public ITextureShareSerialize
{
	// Resource info
	FTextureShareCoreResourceDesc ResourceDesc;

	// GPU-GPU share purpose (this resource use NT handle)
	Windows::HANDLE NTHandle = nullptr;

	// GPU-GPU share purpose (not NT handle)
	Windows::HANDLE SharedHandle = nullptr;

	// Unique handle name
	FGuid SharedHandleGuid;

	// Resource Owner GPU Index
	uint32 GPUIndex = 0;

public:
	virtual ~FTextureShareCoreResourceHandle() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << ResourceDesc << NTHandle << SharedHandle << SharedHandleGuid << GPUIndex;
	}

public:
	bool HandleEquals(const FTextureShareCoreResourceHandle& In) const
	{
		return NTHandle == In.NTHandle
			&& SharedHandleGuid == In.SharedHandleGuid
			&& SharedHandle == In.SharedHandle;
	}

	bool EqualsFunc(const FTextureShareCoreResourceDesc& InResourceDesc) const
	{
		return ResourceDesc.EqualsFunc(InResourceDesc);
	}

	bool EqualsFunc(const FTextureShareCoreViewDesc& InViewDesc) const
	{
		return ResourceDesc.EqualsFunc(InViewDesc);
	}

	bool operator==(const FTextureShareCoreResourceHandle& In) const
	{
		return ResourceDesc == In.ResourceDesc;
	}
};
