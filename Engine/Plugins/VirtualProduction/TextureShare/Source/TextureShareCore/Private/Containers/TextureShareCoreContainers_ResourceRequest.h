// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/TextureShareCoreContainers_ResourceDesc.h"

/**
 * Resource request data
 * When no size and/or format is specified, values from another process or values from local resources are used.
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreResourceRequest
	: public ITextureShareSerialize
{
	// Resource info
	FTextureShareCoreResourceDesc ResourceDesc;

	// MGPU support. When a texture is rendered on a GPU other than the destination,
	// it must be transferred between GPUs.
	// The transfer is performed on the UE side in the TextureShare module.
	int32 GPUIndex = -1;

	// Required texture format
	// The UE process only uses the "EPixelFormat" value.
	// Otherwise, find the best of "EPixelFormat" associated with "DXGI_FORMAT".
	EPixelFormat PixelFormat = EPixelFormat::PF_Unknown;
	DXGI_FORMAT       Format = DXGI_FORMAT_UNKNOWN;

	// Required texture size (or zero if the original value is acceptable)
	FIntPoint           Size = FIntPoint::ZeroValue;

	// Required texture gamma (or negative value if the original value is acceptable)
	float               Gamma = -1;

	// Experimental: nummips feature
	uint32 NumMips = 0;

public:
	virtual ~FTextureShareCoreResourceRequest() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << ResourceDesc << GPUIndex << PixelFormat << Format << Size << Gamma << NumMips;
	}

public:
	FTextureShareCoreResourceRequest() = default;

	FTextureShareCoreResourceRequest(const FTextureShareCoreResourceDesc& InResourceDesc, const float InGamma = -1.f)
		: ResourceDesc(InResourceDesc)
		, Gamma(InGamma)
	{ }

	FTextureShareCoreResourceRequest(const FTextureShareCoreResourceDesc& InResourceDesc, const FIntPoint& InSize, const float InGamma = -1.f)
		: ResourceDesc(InResourceDesc)
		, Size(InSize)
		, Gamma(InGamma)
	{ }

	FTextureShareCoreResourceRequest(const FTextureShareCoreResourceDesc& InResourceDesc, const DXGI_FORMAT InFormat, const float InGamma = -1.f)
		: ResourceDesc(InResourceDesc)
		, Format(InFormat)
		, Gamma(InGamma)
	{ }

	FTextureShareCoreResourceRequest(const FTextureShareCoreResourceDesc& InResourceDesc, const FIntPoint& InSize, const DXGI_FORMAT InFormat, const float InGamma = -1.f)
		: ResourceDesc(InResourceDesc)
		, Format(InFormat)
		, Size(InSize)
		, Gamma(InGamma)
	{ }

	FTextureShareCoreResourceRequest(const FTextureShareCoreResourceDesc& InResourceDesc, const EPixelFormat InPixelFormat, const float InGamma = -1.f)
		: ResourceDesc(InResourceDesc)
		, PixelFormat(InPixelFormat)
		, Gamma(InGamma)
	{ }

	FTextureShareCoreResourceRequest(const FTextureShareCoreResourceDesc& InResourceDesc, const FIntPoint& InSize, const EPixelFormat InPixelFormat, const float InGamma = -1.f)
		: ResourceDesc(InResourceDesc)
		, PixelFormat(InPixelFormat)
		, Size(InSize)
		, Gamma(InGamma)
	{ }

public:


	void SetPixelFormat(uint32 InPixelFormat)
	{
		PixelFormat = (EPixelFormat)InPixelFormat;
	}

	bool EqualsFunc(const FTextureShareCoreResourceDesc& InResourceDesc) const
	{
		return ResourceDesc.EqualsFunc(InResourceDesc);
	}

	bool EqualsFunc(const FTextureShareCoreViewDesc& InViewDesc) const
	{
		return ResourceDesc.EqualsFunc(InViewDesc);
	}

	bool operator==(const FTextureShareCoreResourceRequest& InResourceRequest) const
	{
		return ResourceDesc == InResourceRequest.ResourceDesc;
	}
};
