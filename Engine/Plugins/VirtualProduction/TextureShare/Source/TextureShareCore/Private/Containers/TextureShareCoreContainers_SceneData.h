// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/TextureShareCoreContainers_ViewDesc.h"

/**
 * Scene data: View matrices
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreSceneViewMatrices
	: public ITextureShareSerialize
{
	// ViewToClip : UE projection matrix projects such that clip space Z=1 is the near plane, and Z=0 is the infinite far plane.
	FMatrix ProjectionMatrix;

	// ViewToClipNoAA : UE projection matrix projects such that clip space Z=1 is the near plane, and Z=0 is the infinite far plane. Don't apply any AA jitter
	FMatrix ProjectionNoAAMatrix;

	// WorldToView..
	FMatrix ViewMatrix;

	// WorldToClip : UE projection matrix projects such that clip space Z=1 is the near plane, and Z=0 is the infinite far plane. */
	FMatrix ViewProjectionMatrix;

	// The view-projection transform, starting from world-space points translated by -ViewOrigin.
	FMatrix TranslatedViewProjectionMatrix;

	// The translation to apply to the world before TranslatedViewProjectionMatrix. Usually it is -ViewOrigin but with rereflections this can differ
	FVector PreViewTranslation;

	// To support ortho and other modes this is redundant, in world space
	FVector ViewOrigin;

	// Scale applied by the projection matrix in X and Y.
	FVector2D ProjectionScale;

	// TemporalAA jitter offset currently stored in the projection matrix
	FVector2D TemporalAAProjectionJitter;

	// Scale factor to use when computing the size of a sphere in pixels.
	float ScreenScale = 1.f;

public:
	virtual ~FTextureShareCoreSceneViewMatrices() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << ProjectionMatrix << ProjectionNoAAMatrix
			<< ViewMatrix << ViewProjectionMatrix << TranslatedViewProjectionMatrix << PreViewTranslation
			<< ViewOrigin << ProjectionScale << TemporalAAProjectionJitter << ScreenScale;
	}
};

/**
 * Scene data: View
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreSceneView
	: public ITextureShareSerialize
{
	FTextureShareCoreSceneViewMatrices ViewMatrices;

	// Final position of the view in the final render target (in pixels), potentially constrained by an aspect ratio requirement (black bars)
	FIntRect UnscaledViewRect;

	// Raw view size (in pixels), used for screen space calculations
	FIntRect UnconstrainedViewRect;

	// Variables used to determine the view matrix
	FVector  ViewLocation;
	FRotator ViewRotation;
	FQuat    BaseHmdOrientation;
	FVector  BaseHmdLocation;
	float    WorldToMetersScale = 1.f;

	// For stereoscopic rendering, unique index identifying the view across view families
	int32 StereoViewIndex;

	// For stereoscopic rendering, view family index of the primary view associated with this view
	int32 PrimaryViewIndex;

	// Actual field of view and that desired by the camera originally
	float FOV;
	float DesiredFOV;

public:
	virtual ~FTextureShareCoreSceneView() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << ViewMatrices
			<< UnscaledViewRect << UnconstrainedViewRect
			<< ViewLocation << ViewRotation << BaseHmdOrientation << BaseHmdLocation << WorldToMetersScale
			<< StereoViewIndex << PrimaryViewIndex << FOV << DesiredFOV;
	}
};

/**
 * Scene data: GameTime
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreSceneGameTime
	: public ITextureShareSerialize
{
	float RealTimeSeconds;
	float WorldTimeSeconds;

	float DeltaRealTimeSeconds;
	float DeltaWorldTimeSeconds;

public:
	virtual ~FTextureShareCoreSceneGameTime() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << RealTimeSeconds << WorldTimeSeconds
			<< DeltaRealTimeSeconds << DeltaWorldTimeSeconds;
	}
};

/**
 * Scene data: ViewFamily
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreSceneViewFamily
	: public ITextureShareSerialize
{
	// The current time
	FTextureShareCoreSceneGameTime GameTime;

	// Copy from main thread GFrameNumber to be accessible on render thread side. UINT_MAX before CreateSceneRenderer() or BeginRenderingViewFamily() was called
	uint32 FrameNumber = 0;

	// When enabled, the post processing will output in HDR space
	bool bIsHDR = false;

	// Gamma correction used when rendering this family. Default is 1.0
	float GammaCorrection = 1.f;

	// Secondary view fraction to support High DPI monitor
	float SecondaryViewFraction = 1.f;

public:
	virtual ~FTextureShareCoreSceneViewFamily() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << GameTime << FrameNumber << bIsHDR << GammaCorrection << SecondaryViewFraction;
	}
};

/**
 * Scene data
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreSceneViewData
	: public ITextureShareSerialize
{
	// Eye type of this resource (support stereo)
	FTextureShareCoreViewDesc ViewDesc;

	// Scene view data
	FTextureShareCoreSceneView View;

	// Scene ViewFamily data
	FTextureShareCoreSceneViewFamily ViewFamily;

public:
	virtual ~FTextureShareCoreSceneViewData() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << ViewDesc << View << ViewFamily;
	}

public:
	FTextureShareCoreSceneViewData() = default;

	FTextureShareCoreSceneViewData(const FString& InViewId, const ETextureShareEyeType InEyeType = ETextureShareEyeType::Default)
		: ViewDesc(InViewId, InEyeType)
	{ }

	FTextureShareCoreSceneViewData(const FTextureShareCoreViewDesc& InViewDesc)
		: ViewDesc(InViewDesc)
	{ }

public:
	bool EqualsFunc(const FTextureShareCoreSceneViewData& InViewData) const
	{
		return ViewDesc.EqualsFunc(InViewData.ViewDesc);
	}

	bool EqualsFunc(const FTextureShareCoreViewDesc& InViewDesc) const
	{
		return ViewDesc.EqualsFunc(InViewDesc);
	}

	bool operator==(const FTextureShareCoreSceneViewData& InSceneViewData) const
	{
		return ViewDesc == InSceneViewData.ViewDesc;
	}
};
