// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Serialize/TextureShareCoreSerialize.h"

/**
 * Connection state
 */
enum class ETextureShareCoreFrameConnectionsState : uint8
{
	SkipFrame = 0,
	Wait,
	Accept
};

/**
 * SyncSettings: Connection - Wait until the required number of processes are connected at new frame
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreFrameConnectionsSettings
	: public ITextureShareSerialize
{
	// Required number of processes. The TS frame will only start when this number is reached.
	int32 MinValue = 0;

	// Connect only to processes with names from this list
	TArraySerializable<FString> AllowedProcessNames;

	// Ignore processes with names from this list
	TArraySerializable<FString> BannedProcessNames;

public:
	virtual ~FTextureShareCoreFrameConnectionsSettings() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream& Stream) override
	{
		return Stream << MinValue << AllowedProcessNames << BannedProcessNames;
	}

	bool operator==(const FTextureShareCoreFrameConnectionsSettings& In) const
	{
		return MinValue == In.MinValue
			&& AllowedProcessNames.EqualsFunc(In.AllowedProcessNames)
			&& BannedProcessNames.EqualsFunc(In.BannedProcessNames);
	}

public:
	ETextureShareCoreFrameConnectionsState GetConnectionsState(const int32 InReadyToConnectObjectsCount, const int32 InFrameConnectionsCount) const
	{
		// in case no TS processes and min=0, skip frame immediately
		if (MinValue < 1 && InFrameConnectionsCount < 1)
		{
			return ETextureShareCoreFrameConnectionsState::SkipFrame;
		}

		if (InFrameConnectionsCount == InReadyToConnectObjectsCount)
		{
			if (MinValue < InReadyToConnectObjectsCount)
			{
				return ETextureShareCoreFrameConnectionsState::Accept;
			}
		}

		return ETextureShareCoreFrameConnectionsState::Wait;
	}
};

/**
 * SyncSettings: define all sync steps for this object
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreFrameSyncSettings
	: public ITextureShareSerialize
{
	// Used sync steps by this process (handled by logic above)
	// Order of values: grow
	TArraySerializable<ETextureShareSyncStep> Steps;

public:
	virtual ~FTextureShareCoreFrameSyncSettings() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream& Stream) override
	{
		return Stream << Steps;
	}

public:
	bool EqualsFunc(const FTextureShareCoreFrameSyncSettings& In) const
	{
		return Steps.EqualsFunc(In.Steps);
	}
};

/**
 * SyncSettings: TimeOut values
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreTimeOutSettings
	: public ITextureShareSerialize
{
	// Memory mutex timeout in miliseconds
	uint32 MemoryMutexTimeout = 1000;

	// Timeout for frame begin (process
	// 0 - infinite
	uint32 FrameBeginTimeOut = 0;

	// Max frame sync time, until this frame lost.
	// 0 - infinite
	uint32 FrameSyncTimeOut = 1000;

	// Split time for timeouts. prevent freeze
	uint32 FrameBeginTimeOutSplit = 100;
	uint32 FrameSyncTimeOutSplit = 100;

	// now sync logic threat object as lost, if last access time expired.
	// If the process is alive, it is re-created on-demant and re-connected
	// '-1' - disable this option
	// This value is unique per process.
	int32 ProcessLostStatusTimeOut = 2000;

	// Multithreaded mutex timeout used in FTextureShareCoreObject::LockThread Mutex()
	int32 ThreadMutexTimeout = 1500;

public:
	virtual ~FTextureShareCoreTimeOutSettings() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << MemoryMutexTimeout << FrameBeginTimeOut << FrameSyncTimeOut
			<< FrameBeginTimeOutSplit << FrameSyncTimeOutSplit
			<< ProcessLostStatusTimeOut
			<< ThreadMutexTimeout;
	}

	bool operator==(const FTextureShareCoreTimeOutSettings& In) const
	{
		return MemoryMutexTimeout == In.MemoryMutexTimeout
			&& FrameBeginTimeOut == In.FrameBeginTimeOut
			&& FrameSyncTimeOut == In.FrameSyncTimeOut
			&& FrameBeginTimeOutSplit == In.FrameBeginTimeOutSplit
			&& FrameSyncTimeOutSplit == In.FrameSyncTimeOutSplit
			&& ProcessLostStatusTimeOut == In.ProcessLostStatusTimeOut
			&& ThreadMutexTimeout == In.ThreadMutexTimeout;
	}
};

/**
 * SyncSettings for TextureShareCore object
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreSyncSettings
	: public ITextureShareSerialize
{
	// At the beginning of each frame, the local process waits for the required number of processes.
	// If it is not reached within the specified time, then this frame will not start.
	FTextureShareCoreFrameConnectionsSettings FrameConnectionSettings;

	// Timeout settings for internal IPC sync logic
	FTextureShareCoreTimeOutSettings TimeoutSettings;

	// Frame sync logic settings
	FTextureShareCoreFrameSyncSettings FrameSyncSettings;

public:
	virtual ~FTextureShareCoreSyncSettings() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << FrameConnectionSettings << TimeoutSettings << FrameSyncSettings;
	}

	bool operator==(const FTextureShareCoreSyncSettings& In) const
	{
		return FrameConnectionSettings == (In.FrameConnectionSettings)
			&& TimeoutSettings == In.TimeoutSettings
			&& FrameSyncSettings.EqualsFunc(In.FrameSyncSettings);
	}
};
