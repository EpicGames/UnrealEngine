// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Serialize/TextureShareCoreSerialize.h"
#include "Misc/TextureShareCoreStrings.h"

/**
 * View descriptor (Name + EyeType)
 * The structure is serializable (binary compatible) and reflected on the SDK side (UE types will be replaced with simplified copies from the SDK).
 */
struct FTextureShareCoreViewDesc
	: public ITextureShareSerialize
{
	// Source view name (Debug log purpose)
	FString SrcId;

	// View name
	FString Id = UE::TextureShareCoreStrings::DefaultViewId;

	// Eye type of this view (support stereo)
	ETextureShareEyeType EyeType = ETextureShareEyeType::Default;

public:
	virtual ~FTextureShareCoreViewDesc() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << SrcId << Id << EyeType;
	}

public:
	FTextureShareCoreViewDesc() = default;

	FTextureShareCoreViewDesc(const FTextureShareCoreViewDesc & In)
		: SrcId(In.SrcId), Id(In.Id), EyeType(In.EyeType)
	{ }

	FTextureShareCoreViewDesc(const FString& InSrcViewId, const FString& InViewId, const ETextureShareEyeType InEyeType = ETextureShareEyeType::Default)
		: SrcId(InSrcViewId), Id(InViewId), EyeType(InEyeType)
	{ }

	FTextureShareCoreViewDesc(const FString & InViewId, const ETextureShareEyeType InEyeType = ETextureShareEyeType::Default)
		: Id(InViewId), EyeType(InEyeType)
	{ }

	FTextureShareCoreViewDesc(const ETextureShareEyeType InEyeType)
		: EyeType(InEyeType)
	{ }

	bool EqualsFunc(const FString& InViewId) const
	{
		return InViewId == Id;
	}

	bool EqualsFunc(const FTextureShareCoreViewDesc & InViewDesc) const
	{
		return InViewDesc.Id == Id && EyeType == InViewDesc.EyeType;
	}

	bool operator==(const FTextureShareCoreViewDesc& InViewDesc) const
	{
		return InViewDesc.Id == Id && EyeType == InViewDesc.EyeType;
	}
};
