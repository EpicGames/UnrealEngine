// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if __UNREAL__
#include "CoreMinimal.h"
#endif

/**
 * Render Device interface type
 */
enum class ETextureShareDeviceType : uint8
{
	Undefined = 0,

	D3D11,
	D3D12,

	Vulkan
};

/**
 * TextureShare Process type.
 * This type means how processes are visible to each other.
  */
enum class ETextureShareProcessType : uint8
{
	// Undefined process. Ignored by everyone.
	Undefined = 0,

	// External SDK default process type.
	// Possible connections: [UE], [UE2UE]
	SDK,

	// Unreal Engine default process type.
	// Possible connections: [SDK]
	/**
	* Note: Processes with type [UE] should not be visible to each other.
	* The reason is that multiple instances of Unreal (i.e. nDisplay nodes) start connecting to each other on the same PC if the TextureShare plugin is used.
	* This is because any TextureShare object tries to attach to any suitable remote process.
	* Also, the "invisible" TS object is created by default on startup and starts that sync fight anyway.
	* Therefore, for the UE+UE connection, we use a new process type [UE2UE].
	*/
	UE,

	// Unreal Engine special process type.
	// Possible connections: [SDK], [UE2UE]
	UE2UE,
};

/**
 * Multithread mutex index
 * Using this enum, you can access multi-threaded mutexes.
 */
enum class ETextureShareThreadMutex : uint8
{
	GameThread = 0,
	RenderingThread,

	InternalLock,
	InternalObjectDescMT,
	InternalFrameConnectionsMT,
	InternalSyncSettingsMT,

	COUNT,
	// All mutexes after this dont produce log
	MAX_LOG = InternalLock
};

/**
 * Texture operation
 */
enum class ETextureShareTextureOp : uint8
{
	// Operation not defined (useful to find any type of operation)
	Undefined = 0,

	// Send texture to remote process
	Write,

	// Receive texture
	Read
};

/**
 * View eye type
 */
enum class ETextureShareEyeType : uint8
{
	// Monoscopic
	Default = 0,

	// Stereoscopic rendering
	StereoLeft,
	StereoRight
};

/**
 * Sync steps template type
 */
enum class ETextureShareFrameSyncTemplate : uint8
{
	// Sync logic for BP object
	Default = 0,

	// Minimalistic sync logic for SDK object
	SDK,

	// Sync logic for DC object
	DisplayCluster,

	// Sync logic between DC nodes
	DisplayClusterCrossNode

};

/**
 * Resource type
 */
enum class ETextureShareResourceType: uint8
{
	Default = 0,
	CrossAdapter
};

/**
 * Sync steps values
 */
enum class ETextureShareSyncStep : uint8
{
	Undefined = 0,

	InterprocessConnection,
	
	/**
	 * Frame sync steps (GameThread)
	 */

	FrameBegin,

	FramePreSetupBegin,
	FramePreSetupEnd,

	FrameSetupBegin,
	FrameSetupEnd,

	FramePostSetupBegin,
	FramePostSetupEnd,

	FrameFlush,

	FrameEnd,

	/**
	 * FrameProxy sync steps (RenderThread)
	 */

	FrameProxyBegin,

	FrameSceneFinalColorBegin,
	FrameSceneFinalColorEnd,

	FrameProxyPreRenderBegin,
	FrameProxyPreRenderEnd,

	FrameProxyRenderBegin,
	FrameProxyRenderEnd,

	FrameProxyPostWarpBegin,
	FrameProxyPostWarpEnd,

	FrameProxyPostRenderBegin,
	FrameProxyPostRenderEnd,

	FrameProxyBackBufferReadyToPresentBegin,
	FrameProxyBackBufferReadyToPresentEnd,

	FrameProxyFlush,

	FrameProxyEnd,
};

/**
 * Object sync barrier pass
 */
enum class ETextureShareSyncPass : uint8
{
	Undefined = 0,

	Enter,
	Exit,
	Complete
};

/**
 * Object sync barrier state
 */
enum class ETextureShareSyncState : uint8
{
	Undefined = 0,

	Enter,
	EnterCompleted,

	Exit,
	ExitCompleted,

	Completed,
};

/**
 * Projection matrix source type
 */
enum class ETextureShareCoreSceneViewManualProjectionType : uint8
{
	FrustumAngles = 0,
	Matrix
};

/**
 * Rotator operation type
 * How to change the projection view rotator value
 */
enum class ETextureShareViewRotationDataType : uint8
{
	// Use original value
	Original = 0,

	// Add value from manual projection
	Relative,

	// Use value from manual projection
	Absolute
};

/**
 * Location operation type
 * How to change the projection view location value
 */
enum class ETextureShareViewLocationDataType : uint8
{
	// Use original value
	Original = 0,

	// Add value from manual projection
	Relative,

	// Use value from manual projection
	Absolute,

	// Use original value, but without stereo view offset
	Original_NoViewOffset,

	// Add value from manual projection, but without stereo view offset
	Relative_NoViewOffset,

	// Use value from manual projection, but without stereo view offset
	Absolute_NoViewOffset
};
