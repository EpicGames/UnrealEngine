// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/ResourcesCache/D3D11/TextureShareCoreD3D11ResourcesCache.h"
#include "Core/ResourcesCache/D3D11/TextureShareCoreD3D11ResourcesCacheItem.h"
#include "Core/TextureShareCoreSecurityAttributes.h"

#include "Module/TextureShareCoreLog.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreD3D11ResourcesCache
//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreD3D11ResourcesCache::CreateSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D11Device* InD3D11Device, ID3D11Texture2D* InResourceD3D11, const FTextureShareCoreResourceDesc& InResourceDesc, FTextureShareCoreResourceHandle& OutResourceHandle)
{
	FObjectCachedResources& ObjectResources = GetObjectCachedResources(InObjectDesc);

	// Use exist resource from cache
	if (FTextureShareCoreResourcesCacheItem* ExistResourcesCacheItem = ObjectResources.FindByNativeResourcePtr(InResourceD3D11))
	{
		OutResourceHandle = ExistResourcesCacheItem->GetHandle();

		check(OutResourceHandle.EqualsFunc(InResourceDesc));

		return true;
	}

	// Create new resource
	TSharedPtr<FTextureShareCoreResourcesCacheItem> NewResourcesCacheItem = MakeShared<FTextureShareCoreD3D11ResourcesCacheItem>(InResourceD3D11, InResourceDesc, SecurityAttributes->GetSecurityAttributes(ETextureShareSecurityAttributesType::Resource));
	if (NewResourcesCacheItem.IsValid() && NewResourcesCacheItem->GetNativeResource())
	{
		ObjectResources.Resources.Add(NewResourcesCacheItem);
		OutResourceHandle = NewResourcesCacheItem->GetHandle();
		
		return true;
	}

	return false;
}

ID3D11Texture2D* FTextureShareCoreD3D11ResourcesCache::OpenSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D11Device* InD3D11Device, const FTextureShareCoreResourceHandle& InResourceHandle)
{
	FObjectCachedResources& ObjectResources = GetObjectCachedResources(InObjectDesc);

	// Use exist resource from cache
	if (FTextureShareCoreResourcesCacheItem* ExistResourcesCacheItem = ObjectResources.Find(InResourceHandle))
	{
		return static_cast<ID3D11Texture2D*>(ExistResourcesCacheItem->GetNativeResource());
	}

	// Create new resource
	TSharedPtr<FTextureShareCoreResourcesCacheItem> NewResourcesCacheItem = MakeShared<FTextureShareCoreD3D11ResourcesCacheItem>(InD3D11Device, InResourceHandle);
	if (NewResourcesCacheItem.IsValid() && NewResourcesCacheItem->GetNativeResource())
	{
		ObjectResources.Resources.Add(NewResourcesCacheItem);

		return static_cast<ID3D11Texture2D*>(NewResourcesCacheItem->GetNativeResource());
	}

	return nullptr;
}

bool FTextureShareCoreD3D11ResourcesCache::RemoveSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D11Texture2D* InResourceD3D11)
{
	return GetObjectCachedResources(InObjectDesc).RemoveByNativeResourcePtr(InResourceD3D11);
}
