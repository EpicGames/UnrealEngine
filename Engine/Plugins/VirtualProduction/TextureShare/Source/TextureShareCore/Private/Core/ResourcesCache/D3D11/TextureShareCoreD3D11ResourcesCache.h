// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ITextureShareCoreD3D11ResourcesCache.h"
#include "Containers/TextureShareCoreContainers_DeviceD3D11.h"
#include "Core/ResourcesCache/TextureShareCoreResourcesCache.h"

class FTextureShareCoreSecurityAttributes;

/**
 * D3D11 renderer resources cache container
 */
class FTextureShareCoreD3D11ResourcesCache
	: public ITextureShareCoreD3D11ResourcesCache
	, public FTextureShareCoreResourcesCache
{
public:
	FTextureShareCoreD3D11ResourcesCache(const TSharedPtr<FTextureShareCoreSecurityAttributes>& InSecurityAttributes)
		: SecurityAttributes(InSecurityAttributes)
	{ }
	virtual ~FTextureShareCoreD3D11ResourcesCache() = default;

public:
	virtual bool CreateSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D11Device* InD3D11Device, ID3D11Texture2D* InResourceD3D11, const FTextureShareCoreResourceDesc& InResourceDesc, FTextureShareCoreResourceHandle& OutResourceHandle) override;
	virtual ID3D11Texture2D* OpenSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D11Device* InD3D11Device, const FTextureShareCoreResourceHandle& InResourceHandle) override;
	virtual bool RemoveSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D11Texture2D* InResourceD3D11) override;

private:
	TSharedPtr<FTextureShareCoreSecurityAttributes> SecurityAttributes;
};
