// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/ResourcesCache/D3D11/TextureShareCoreD3D11ResourcesCacheItem.h"
#include "Module/TextureShareCoreLog.h"
#include "Templates/RefCounting.h"

#include "Windows/D3D11ThirdParty.h"

namespace UE::TextureShareCore::D3D11ResourcesCacheItem
{
	static inline const FString GetD3D11ComErrorDescription(HRESULT Res)
	{
		const uint32 BufSize = 4096;
		WIDECHAR buffer[4096];
		if (::FormatMessageW(FORMAT_MESSAGE_FROM_SYSTEM,
			nullptr,
			Res,
			MAKELANGID(LANG_NEUTRAL, SUBLANG_NEUTRAL),
			buffer,
			sizeof(buffer) / sizeof(*buffer),
			nullptr))
		{
			return buffer;
		}
		else
		{
			return TEXT("[cannot find d3d11 error description]");
		}
	}

	static inline FString GetUniqueD3D11ResourceHandleName(const FGuid& InResourceGuid)
	{
		return FString::Printf(TEXT("Global\\%s"), *InResourceGuid.ToString(EGuidFormats::DigitsWithHyphensInBraces));
	}

	static bool D3D11CreateSharedNTHandle(ID3D11Texture2D* InResourceD3D11, const void* InSecurityAttributes, FGuid& OutResourceHandleGuid, HANDLE& OutSharedNTHandle)
	{
		TRefCountPtr<IDXGIResource1> DXGIResource1;
		{
			// Open DXGIResource1 interface:
			HRESULT Result = InResourceD3D11->QueryInterface(IID_PPV_ARGS(DXGIResource1.GetInitReference()));
			if (FAILED(Result))
			{
				UE_LOGF(LogTextureShareCoreD3D, Error, "D3D11CreateNamedNTHandle::QueryInterface failed: 0x%X - %ls", Result, *GetD3D11ComErrorDescription(Result));

				return false;
			}
		}

		if (DXGIResource1.IsValid())
		{
			OutResourceHandleGuid = FGuid::NewGuid();

			// Microsoft docs: Currently the only value this parameter accepts is GENERIC_ALL.
			const DWORD dwAccess = GENERIC_ALL;

			// Create new shared handle
			HRESULT Result = DXGIResource1->CreateSharedHandle((const SECURITY_ATTRIBUTES*)InSecurityAttributes, dwAccess, *GetUniqueD3D11ResourceHandleName(OutResourceHandleGuid), &OutSharedNTHandle);
			if (FAILED(Result))
			{
				UE_LOGF(LogTextureShareCoreD3D, Error, "D3D11CreateNamedNTHandle::CreateSharedHandle failed: 0x%X - %ls", Result, *GetD3D11ComErrorDescription(Result));

				return false;
			}

			return OutSharedNTHandle != NULL;
		}

		UE_LOGF(LogTextureShareCoreD3D, Error, "D3D11CreateNamedNTHandle::DXGIResource1 invalid");

		return false;
	}

	static bool D3D11CreateSharedHandle(ID3D11Texture2D* InResourceD3D11, HANDLE& OutSharedHandle)
	{
		// Use old D3D11 shared resources interfaces
		TRefCountPtr<IDXGIResource> DXGIResource;
		{
			HRESULT Result = InResourceD3D11->QueryInterface(IID_PPV_ARGS(DXGIResource.GetInitReference()));
			if (FAILED(Result))
			{
				UE_LOGF(LogTextureShareCoreD3D, Error, "D3D11CreateSharedHandle::QueryInterface failed: 0x%X - %ls", Result, *GetD3D11ComErrorDescription(Result));

				return false;
			}
		}

		if (DXGIResource.IsValid())
		{
			//
			// NOTE : The HANDLE IDXGIResource::GetSharedHandle gives us is NOT an NT Handle, and therefre we should not call CloseHandle on it
			//
			HRESULT Result = DXGIResource->GetSharedHandle(&OutSharedHandle);
			if (FAILED(Result))
			{
				UE_LOGF(LogTextureShareCoreD3D, Error, "D3D11CreateSharedHandle::GetSharedHandle failed: 0x%X - %ls", Result, *GetD3D11ComErrorDescription(Result));

				return false;
			}

			return OutSharedHandle != NULL;
		}

		UE_LOGF(LogTextureShareCoreD3D, Error, "D3D11CreateSharedHandle::DXGIResource invalid");

		return false;
	}

	static ID3D11Texture2D* D3D11OpenSharedHandle(ID3D11Device* InD3D11Device, const HANDLE InSharedResourceHandle)
	{
		// Open not NT Handle
		if (InD3D11Device && InSharedResourceHandle)
		{
			ID3D11Texture2D* D3D11Resource = nullptr;
			HRESULT Result = InD3D11Device->OpenSharedResource(InSharedResourceHandle, __uuidof(ID3D11Texture2D), (LPVOID*)&D3D11Resource);
			if (SUCCEEDED(Result))
			{
				return D3D11Resource;
			}

			UE_LOGF(LogTextureShareCoreD3D, Error, "D3D11OpenSharedResourceFromHandle::OpenSharedResource failed: 0x%X - %ls", Result, *GetD3D11ComErrorDescription(Result));
		}

		return nullptr;
	}

	static ID3D11Texture2D* D3D11OpenSharedNTHandle(ID3D11Device* InD3D11Device, const FGuid& InResourceHandleGuid, const HANDLE& InSharedNTHandle)
	{
		const bool bOpenNamedHandle = InResourceHandleGuid.IsValid();
		if (bOpenNamedHandle || InSharedNTHandle)
		{
			// Gives a device access to a shared resource that is referenced by a handle and that was created on a different device.
			// You must have previously created the resource as shared and specified that it uses NT handles
			// (that is, you set the D3D11_RESOURCE_MISC_SHARED_NTHANDLE flag).
			TRefCountPtr <ID3D11Device1> D3D11Device1;
			{
				HRESULT Result = InD3D11Device->QueryInterface(__uuidof(ID3D11Device1), (void**)D3D11Device1.GetInitReference());
				if (FAILED(Result))
				{
					UE_LOGF(LogTextureShareCoreD3D, Error, "D3D11OpenSharedNTHandle::QueryInterface failed: 0x%X - %ls", Result, *GetD3D11ComErrorDescription(Result));

					return nullptr;
				}
			}

			if (D3D11Device1.IsValid())
			{
				ID3D11Texture2D* D3D11Resource = nullptr;

				if (bOpenNamedHandle)
				{
					const DWORD dwDesiredAccess = DXGI_SHARED_RESOURCE_READ | DXGI_SHARED_RESOURCE_WRITE;

					HRESULT Result = D3D11Device1->OpenSharedResourceByName(*GetUniqueD3D11ResourceHandleName(InResourceHandleGuid), dwDesiredAccess, __uuidof(ID3D11Texture2D), (LPVOID*)&D3D11Resource);
					if (SUCCEEDED(Result))
					{
						return D3D11Resource;
					}

					UE_LOGF(LogTextureShareCoreD3D, Warning, "D3D11OpenSharedNTHandle::OpenSharedResourceByName failed: 0x%X - %ls", Result, *GetD3D11ComErrorDescription(Result));
					// try to open from NTHandle
				}

				if (InSharedNTHandle)
				{
					HRESULT Result = D3D11Device1->OpenSharedResource1(InSharedNTHandle, __uuidof(ID3D11Texture2D), (LPVOID*)&D3D11Resource);
					if (SUCCEEDED(Result))
					{
						return D3D11Resource;
					}

					UE_LOGF(LogTextureShareCoreD3D, Error, "D3D11OpenSharedNTHandle::OpenSharedResource1 failed: 0x%X - %ls", Result, *GetD3D11ComErrorDescription(Result));
				}
			}
		}

		return nullptr;
	}
};
using namespace UE::TextureShareCore::D3D11ResourcesCacheItem;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreD3D11ResourcesCacheItem
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareCoreD3D11ResourcesCacheItem::~FTextureShareCoreD3D11ResourcesCacheItem()
{
	if (bNeedReleaseD3D11Resource && D3D11Resource)
	{
		D3D11Resource->Release();
		D3D11Resource = nullptr;
	}
}

// Create shared resource
FTextureShareCoreD3D11ResourcesCacheItem::FTextureShareCoreD3D11ResourcesCacheItem(ID3D11Texture2D* InResourceD3D11, const FTextureShareCoreResourceDesc& InResourceDesc, const void* InSecurityAttributes)
{
	if (InResourceD3D11)
	{
		D3D11_TEXTURE2D_DESC InDesc;
		InResourceD3D11->GetDesc(&InDesc);

		// Microsoft docs: Starting with Direct3D 11.1, we recommend not to use GetSharedHandle anymore to retrieve the handle to a shared resource. 
		// Instead, use IDXGIResource1::CreateSharedHandle to get a handle for sharing.
		// To use IDXGIResource1::CreateSharedHandle, you must create the resource as shared and specify that it uses NT handles
		// (that is, you set the D3D11_RESOURCE_MISC_SHARED_NTHANDLE flag). We also recommend that you create shared resources 
		// that use NT handles so you can use CloseHandle, DuplicateHandle, and so on on those shared resources.)

		// Microsoft docs: you must create the resource as shared and specify that it uses NT handles (that is, you set the D3D11_RESOURCE_MISC_SHARED_NTHANDLE flag).
		// We also recommend that you create shared resources that use NT handles so you can use CloseHandle, DuplicateHandle, and so on on those shared resources.]
		bool bResourceUseNTHandle = (InDesc.MiscFlags & D3D11_RESOURCE_MISC_SHARED_NTHANDLE) != 0;
		if (bResourceUseNTHandle)
		{
			FGuid ResourceHandleGuid;
			HANDLE SharedNTHandle;
			if (D3D11CreateSharedNTHandle(InResourceD3D11, InSecurityAttributes, ResourceHandleGuid, SharedNTHandle))
			{
				// Created NT handles need release at end
				bNeedReleaseNTHandle = true;

				Handle.ResourceDesc= InResourceDesc;
				Handle.NTHandle = SharedNTHandle;
				Handle.SharedHandleGuid = ResourceHandleGuid;
				Handle.SharedHandle = NULL;

				// Store valid texture ptr
				D3D11Resource = InResourceD3D11;

				return;
			}
		}

		// Use old D3D11 shared resources interfaces
		HANDLE SharedHandle;
		if(D3D11CreateSharedHandle(InResourceD3D11, SharedHandle))
		{
			Handle.ResourceDesc = InResourceDesc;
			Handle.NTHandle = NULL;
			Handle.SharedHandleGuid = FGuid();
			Handle.SharedHandle = SharedHandle;

			// Store valid texture ptr
			D3D11Resource = InResourceD3D11;
		}
	}
}

// Open shared resource
FTextureShareCoreD3D11ResourcesCacheItem::FTextureShareCoreD3D11ResourcesCacheItem(ID3D11Device* InD3D11Device, const FTextureShareCoreResourceHandle& InResourceHandle)
{
	Handle = InResourceHandle;

	if (InD3D11Device)
	{
		// Open not NT Handle
		if (Handle.SharedHandle)
		{
			D3D11Resource = D3D11OpenSharedHandle(InD3D11Device, Handle.SharedHandle);
		}

		// Open NT handle
		if (D3D11Resource == nullptr)
		{
			D3D11Resource = D3D11OpenSharedNTHandle(InD3D11Device, Handle.SharedHandleGuid, Handle.NTHandle);
		}

		// Release opened D3D11 resource in destructor
		bNeedReleaseD3D11Resource = (D3D11Resource != nullptr);
	}
}
