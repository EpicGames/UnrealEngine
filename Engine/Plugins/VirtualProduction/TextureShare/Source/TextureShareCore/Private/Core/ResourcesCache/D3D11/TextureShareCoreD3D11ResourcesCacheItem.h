// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Core/ResourcesCache/TextureShareCoreResourcesCacheItem.h"
#include "Containers/TextureShareCoreContainers_DeviceD3D11.h"

/**
 * D3D11 renderer resource container
 */
class FTextureShareCoreD3D11ResourcesCacheItem
	: public FTextureShareCoreResourcesCacheItem
{
public:
	// Create shared resource
	FTextureShareCoreD3D11ResourcesCacheItem(ID3D11Texture2D* InResourceD3D11, const FTextureShareCoreResourceDesc& InResourceDesc, const void* InSecurityAttributes);

	// Open shared resource
	FTextureShareCoreD3D11ResourcesCacheItem(ID3D11Device* InD3D11Device, const FTextureShareCoreResourceHandle& InResourceHandle);

	virtual ~FTextureShareCoreD3D11ResourcesCacheItem();

	virtual void* GetNativeResource() const override
	{
		return D3D11Resource;
	}

public:
	// Opened from Handle resources released
	bool bNeedReleaseD3D11Resource = false;

	// Saved resource ptr
	ID3D11Texture2D* D3D11Resource = nullptr;
};
