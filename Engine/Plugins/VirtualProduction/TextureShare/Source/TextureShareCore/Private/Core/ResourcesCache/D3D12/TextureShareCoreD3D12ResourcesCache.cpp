// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/ResourcesCache/D3D12/TextureShareCoreD3D12ResourcesCache.h"
#include "Core/ResourcesCache/D3D12/TextureShareCoreD3D12ResourcesCacheItem.h"
#include "Core/TextureShareCoreSecurityAttributes.h"

#include "Module/TextureShareCoreLog.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreD3D12ResourcesCache
//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreD3D12ResourcesCache::CreateSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D12Device* InD3D12Device, ID3D12Resource* InResourceD3D12, const uint32 InResourceGPUIndex, const FTextureShareCoreResourceDesc& InResourceDesc, FTextureShareCoreResourceHandle& OutResourceHandle)
{
	FObjectCachedResources& ObjectResources = GetObjectCachedResources(InObjectDesc);

	// Use exist resource from cache
	if (FTextureShareCoreResourcesCacheItem* ExistResourcesCacheItem = ObjectResources.FindByNativeResourcePtr(InResourceD3D12))
	{
		OutResourceHandle = ExistResourcesCacheItem->GetHandle();

		check(OutResourceHandle.EqualsFunc(InResourceDesc));

		return true;
	}

	// Create new shared resource
	TSharedPtr<FTextureShareCoreD3D12ResourcesCacheItem> NewResourcesCacheItem = MakeShared<FTextureShareCoreD3D12ResourcesCacheItem>(InD3D12Device, InResourceD3D12, InResourceDesc, SecurityAttributes->GetSecurityAttributes(ETextureShareSecurityAttributesType::Resource));
	if (NewResourcesCacheItem.IsValid() && NewResourcesCacheItem->GetNativeResource())
	{
		// Support mGPU
		NewResourcesCacheItem->SetResourceHandleGPUIndex(InResourceGPUIndex);

		ObjectResources.Resources.Add(NewResourcesCacheItem);
		OutResourceHandle = NewResourcesCacheItem->GetHandle();

		return true;
	}

	return false;
}

ID3D12Resource* FTextureShareCoreD3D12ResourcesCache::CreateCrossAdapterResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D12Device* InD3D12Device, const int32 InWidth, const int32 InHeight, const DXGI_FORMAT InFormat, const FTextureShareCoreResourceDesc& InResourceDesc, FTextureShareCoreResourceHandle& OutResourceHandle)
{
	FObjectCachedResources& ObjectResources = GetObjectCachedResources(InObjectDesc);

	// Use exist resource from cache
	if (FTextureShareCoreResourcesCacheItem* ExistResourcesCacheItem = ObjectResources.FindByResourceDesc(InResourceDesc))
	{
		OutResourceHandle = ExistResourcesCacheItem->GetHandle();

		check(OutResourceHandle.EqualsFunc(InResourceDesc));

		return (ID3D12Resource*)ExistResourcesCacheItem->GetNativeResource();
	}

	// Create new cross-adapter resource
	TSharedPtr<FTextureShareCoreD3D12ResourcesCacheItem> NewResourcesCacheItem = MakeShared<FTextureShareCoreD3D12ResourcesCacheItem>(InD3D12Device, InWidth, InHeight, InFormat, InResourceDesc, SecurityAttributes->GetSecurityAttributes(ETextureShareSecurityAttributesType::Resource));
	if (NewResourcesCacheItem.IsValid() && NewResourcesCacheItem->GetNativeResource())
	{
		ObjectResources.Resources.Add(NewResourcesCacheItem);
		OutResourceHandle = NewResourcesCacheItem->GetHandle();

		return NewResourcesCacheItem->D3D12Resource;
	}

	return nullptr;
}

ID3D12Resource* FTextureShareCoreD3D12ResourcesCache::OpenSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D12Device* InD3D12Device, const FTextureShareCoreResourceHandle& InResourceHandle)
{
	FObjectCachedResources& ObjectResources = GetObjectCachedResources(InObjectDesc);

	// Use exist resource from cache
	if (FTextureShareCoreResourcesCacheItem* ExistResourcesCacheItem = ObjectResources.Find(InResourceHandle))
	{
		return static_cast<ID3D12Resource*>(ExistResourcesCacheItem->GetNativeResource());
	}

	// Create new resource
	TSharedPtr<FTextureShareCoreResourcesCacheItem> NewResourcesCacheItem = MakeShared<FTextureShareCoreD3D12ResourcesCacheItem>(InD3D12Device, InResourceHandle);
	if (NewResourcesCacheItem.IsValid() && NewResourcesCacheItem->GetNativeResource())
	{
		ObjectResources.Resources.Add(NewResourcesCacheItem);

		return static_cast<ID3D12Resource*>(NewResourcesCacheItem->GetNativeResource());
	}

	return nullptr;
}

bool FTextureShareCoreD3D12ResourcesCache::RemoveSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D12Resource* InResourceD3D12)
{
	return GetObjectCachedResources(InObjectDesc).RemoveByNativeResourcePtr(InResourceD3D12);
}
