// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ITextureShareCoreD3D12ResourcesCache.h"
#include "Containers/TextureShareCoreContainers_DeviceD3D12.h"
#include "Core/ResourcesCache/TextureShareCoreResourcesCache.h"

class FTextureShareCoreSecurityAttributes;

/**
 * D3D12 renderer resources cache container
 */
class FTextureShareCoreD3D12ResourcesCache
	: public ITextureShareCoreD3D12ResourcesCache
	, public FTextureShareCoreResourcesCache
{
public:
	FTextureShareCoreD3D12ResourcesCache(const TSharedPtr<FTextureShareCoreSecurityAttributes>& InSecurityAttributes)
		: SecurityAttributes(InSecurityAttributes)
	{ }
	virtual ~FTextureShareCoreD3D12ResourcesCache() = default;

public:
	virtual ID3D12Resource* CreateCrossAdapterResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D12Device* InD3D12Device, const int32 InWidth, const int32 InHeight, const DXGI_FORMAT InFormat, const FTextureShareCoreResourceDesc& InResourceDesc, FTextureShareCoreResourceHandle& OutResourceHandle) override;

	virtual bool CreateSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D12Device* InD3D12Device, ID3D12Resource* InResourceD3D12, const uint32 InResourceGPUIndex, const FTextureShareCoreResourceDesc& InResourceDesc, FTextureShareCoreResourceHandle& OutResourceHandle) override;
	virtual ID3D12Resource* OpenSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D12Device* InD3D12Device, const FTextureShareCoreResourceHandle& InResourceHandle) override;
	virtual bool RemoveSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D12Resource* InResourceD3D12) override;
private:
	TSharedPtr<FTextureShareCoreSecurityAttributes> SecurityAttributes;
};
