// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/ResourcesCache/D3D12/TextureShareCoreD3D12ResourcesCacheItem.h"
#include "Module/TextureShareCoreLog.h"
#include "Templates/RefCounting.h"

#include "D3D12ThirdParty.h"

namespace UE::TextureShareCore::D3D12ResourcesCacheItem
{
	static inline const FString GetD3D12ComErrorDescription(HRESULT Res)
	{
		const uint32 BufSize = 4096;
		WIDECHAR buffer[4096];
		if (::FormatMessageW(FORMAT_MESSAGE_FROM_SYSTEM,
			nullptr,
			Res,
			MAKELANGID(LANG_NEUTRAL, SUBLANG_NEUTRAL),
			buffer,
			sizeof(buffer) / sizeof(*buffer),
			nullptr))
		{
			return buffer;
		}
		else
		{
			return TEXT("[cannot find d3d12 error description]");
		}
	}

	static inline FString GetUniqueD3D12ResourceHandleName(const FGuid& InResourceGuid)
	{
		return FString::Printf(TEXT("Global\\%s"), *InResourceGuid.ToString(EGuidFormats::DigitsWithHyphensInBraces));
	}
};
using namespace UE::TextureShareCore::D3D12ResourcesCacheItem;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreD3D12ResourceCache::FCachedResourceD3D12
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareCoreD3D12ResourcesCacheItem::~FTextureShareCoreD3D12ResourcesCacheItem()
{
	if (bNeedReleaseD3D12Resource && D3D12Resource)
	{
		D3D12Resource->Release();
		D3D12Resource = nullptr;
	}
}

// Create shared resource
void FTextureShareCoreD3D12ResourcesCacheItem::ImplCreateSharedResource(ID3D12Device* InD3D12Device, ID3D12Resource* InResourceD3D12, const FTextureShareCoreResourceDesc& InResourceDesc, const void* InSecurityAttributes)
{
	if (InD3D12Device && InResourceD3D12)
	{
		FGuid ResourceHandleGuid = FGuid::NewGuid();

		// Microsoft docs: Currently the only value this parameter accepts is GENERIC_ALL.
		const DWORD dwAccess = GENERIC_ALL;

		//NT HANDLE value to the resource to share. You can use this handle in calls to access the resource.
		HANDLE SharedNTHandle;
		{
			HRESULT Result = InD3D12Device->CreateSharedHandle(InResourceD3D12, (const SECURITY_ATTRIBUTES*)InSecurityAttributes, dwAccess, *GetUniqueD3D12ResourceHandleName(ResourceHandleGuid), &SharedNTHandle);
			if (FAILED(Result))
			{
				UE_LOGF(LogTextureShareCoreD3D, Error, "D3D12CreateSharedHandle failed: 0x%X - %ls", Result, *GetD3D12ComErrorDescription(Result));
				return;
			}
		}

		if (SharedNTHandle)
		{
			bNeedReleaseNTHandle = true;

			Handle.ResourceDesc = InResourceDesc;
			Handle.SharedHandleGuid = ResourceHandleGuid;
			Handle.NTHandle = SharedNTHandle;
			Handle.SharedHandle = NULL;

			// Store valid texture ptr
			D3D12Resource = InResourceD3D12;
		}
	}
}

void FTextureShareCoreD3D12ResourcesCacheItem::ImplCreateSharedCrossAdapterResource(ID3D12Device* InD3D12Device, const int32 InWidth, const int32 InHeight, const DXGI_FORMAT InFormat, const FTextureShareCoreResourceDesc& InResourceDesc, const void* InSecurityAttributes)
{
	const uint32 ActualMSAACount = 1;
	const uint32 ActualMSAAQuality = 0;

	D3D12_RESOURCE_DESC CrossAdapterResourceDesc = CD3DX12_RESOURCE_DESC::Tex2D(
		InFormat,
		InWidth,
		InHeight,
		1,  // Array size
		1,
		ActualMSAACount,
		ActualMSAAQuality,
		D3D12_RESOURCE_FLAG_NONE);  // Add misc flags later

	CrossAdapterResourceDesc.Flags = D3D12_RESOURCE_FLAG_ALLOW_CROSS_ADAPTER | D3D12_RESOURCE_FLAG_ALLOW_SIMULTANEOUS_ACCESS;
	CrossAdapterResourceDesc.Layout = D3D12_TEXTURE_LAYOUT_ROW_MAJOR;

	CD3DX12_HEAP_PROPERTIES HeapProperty(D3D12_HEAP_TYPE_DEFAULT);

	ID3D12Resource* D3D12CrossAdapterResource = nullptr;

	HRESULT Result = InD3D12Device->CreateCommittedResource(
		&HeapProperty,
		D3D12_HEAP_FLAG_SHARED | D3D12_HEAP_FLAG_SHARED_CROSS_ADAPTER,
		&CrossAdapterResourceDesc,
		D3D12_RESOURCE_STATE_PIXEL_SHADER_RESOURCE,
		nullptr,
		IID_PPV_ARGS(&D3D12CrossAdapterResource)
	);

	if (SUCCEEDED(Result))
	{
		ImplCreateSharedResource(InD3D12Device, D3D12CrossAdapterResource, InResourceDesc, InSecurityAttributes);
		bNeedReleaseD3D12Resource = true;

		return;
	}

	UE_LOGF(LogTextureShareCoreD3D, Error, "D3D12CreateCrossAdapterResource failed: 0x%X - %ls", Result, *GetD3D12ComErrorDescription(Result));
}

// Open shared resource
void FTextureShareCoreD3D12ResourcesCacheItem::ImplOpenSharedResource(ID3D12Device* InD3D12Device, const FTextureShareCoreResourceHandle& InResourceHandle)
{
	Handle = InResourceHandle;

	if (InD3D12Device)
	{
		if (Handle.SharedHandleGuid.IsValid())
		{
			HANDLE NamedNTHandleHandle = nullptr;
			{
				HRESULT Result = InD3D12Device->OpenSharedHandleByName(*GetUniqueD3D12ResourceHandleName(InResourceHandle.SharedHandleGuid), GENERIC_ALL, &NamedNTHandleHandle);
				if (FAILED(Result))
				{
					UE_LOGF(LogTextureShareCoreD3D, Error, "D3D12OpenSharedHandleByName failed: 0x%X - %ls", Result, *GetD3D12ComErrorDescription(Result));
					return;
				}
			}

			if (NamedNTHandleHandle)
			{
				HRESULT Result = InD3D12Device->OpenSharedHandle(NamedNTHandleHandle, __uuidof(ID3D12Resource), (LPVOID*)&D3D12Resource);
				CloseHandle(NamedNTHandleHandle);

				if (FAILED(Result))
				{
					UE_LOGF(LogTextureShareCoreD3D, Error, "D3D12OpenSharedHandle failed: 0x%X - %ls", Result, *GetD3D12ComErrorDescription(Result));
					return;
				}
			}

		}

		if (D3D12Resource == nullptr && InResourceHandle.NTHandle)
		{
			HRESULT Result = InD3D12Device->OpenSharedHandle(InResourceHandle.NTHandle, __uuidof(ID3D12Resource), (LPVOID*)&D3D12Resource);
			
			if (FAILED(Result))
			{
				UE_LOGF(LogTextureShareCoreD3D, Error, "D3D12OpenSharedNTHandle failed: 0x%X - %ls", Result, *GetD3D12ComErrorDescription(Result));
				return;
			}
		}

		if (D3D12Resource == nullptr && InResourceHandle.SharedHandle)
		{
			HRESULT Result = InD3D12Device->OpenSharedHandle(InResourceHandle.SharedHandle, __uuidof(ID3D12Resource), (LPVOID*)&D3D12Resource);

			if (FAILED(Result))
			{
				UE_LOGF(LogTextureShareCoreD3D, Error, "D3D12OpenSharedHandle failed: 0x%X - %ls", Result, *GetD3D12ComErrorDescription(Result));
				return;
			}
		}

		// Release opened D3D12 resource in destructor
		bNeedReleaseD3D12Resource = D3D12Resource != nullptr;
	}
}
