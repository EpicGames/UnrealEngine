// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/TextureShareCoreContainers_DeviceD3D12.h"
#include "Core/ResourcesCache/TextureShareCoreResourcesCacheItem.h"

/**
 * D3D12 renderer resource container
 */
class FTextureShareCoreD3D12ResourcesCacheItem
	: public FTextureShareCoreResourcesCacheItem
{
public:
	// Create shared resource
	FTextureShareCoreD3D12ResourcesCacheItem(ID3D12Device* InD3D12Device, ID3D12Resource* InResourceD3D12, const FTextureShareCoreResourceDesc& InResourceDesc, const void* InSecurityAttributes)
	{
		ImplCreateSharedResource(InD3D12Device, InResourceD3D12, InResourceDesc, InSecurityAttributes);
	}

	// Create cross-adapter shared resource
	FTextureShareCoreD3D12ResourcesCacheItem(ID3D12Device* InD3D12Device, const int32 InWidth, const int32 InHeight, const DXGI_FORMAT InFormat, const FTextureShareCoreResourceDesc& InResourceDesc, const void* InSecurityAttributes)
	{
		ImplCreateSharedCrossAdapterResource(InD3D12Device, InWidth, InHeight, InFormat, InResourceDesc, InSecurityAttributes);
	}

	// Open shared resource
	FTextureShareCoreD3D12ResourcesCacheItem(ID3D12Device* InD3D12Device, const FTextureShareCoreResourceHandle& InResourceHandle)
	{
		ImplOpenSharedResource(InD3D12Device, InResourceHandle);
	}

	virtual ~FTextureShareCoreD3D12ResourcesCacheItem();

	virtual void* GetNativeResource() const override
	{
		return D3D12Resource;
	}

private:
	void ImplCreateSharedResource(ID3D12Device* InD3D12Device, ID3D12Resource* InResourceD3D12, const FTextureShareCoreResourceDesc& InResourceDesc, const void* InSecurityAttributes);
	void ImplCreateSharedCrossAdapterResource(ID3D12Device* InD3D12Device, const int32 InWidth, const int32 InHeight, const DXGI_FORMAT InFormat, const FTextureShareCoreResourceDesc& InResourceDesc, const void* InSecurityAttributes);
	void ImplOpenSharedResource(ID3D12Device* InD3D12Device, const FTextureShareCoreResourceHandle& InResourceHandle);

public:
	// Opened from Handle resources released
	bool bNeedReleaseD3D12Resource = false;

	// Saved resource ptr
	ID3D12Resource* D3D12Resource = nullptr;
};
