// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/ResourcesCache/TextureShareCoreResourcesCache.h"
#include "Module/TextureShareCoreLog.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreResourcesCache::FObjectCachedResources
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareCoreResourcesCacheItem* FTextureShareCoreResourcesCache::FObjectCachedResources::Find(const FTextureShareCoreResourceHandle& InResourceHandle)
{
	const int32 Index = Resources.IndexOfByPredicate([InResourceHandle](const TSharedPtr<FTextureShareCoreResourcesCacheItem>& CachedResourcePtrIt)
	{
		return CachedResourcePtrIt.IsValid() && CachedResourcePtrIt->HandleEquals(InResourceHandle);
	});

	if (Index != INDEX_NONE)
	{
		return Resources[Index].Get();
	}

	return nullptr;
}

FTextureShareCoreResourcesCacheItem* FTextureShareCoreResourcesCache::FObjectCachedResources::FindByNativeResourcePtr(const void* InNativeResourcePtr)
{
	if (InNativeResourcePtr)
	{
		const int32 Index = Resources.IndexOfByPredicate([InNativeResourcePtr](const TSharedPtr<FTextureShareCoreResourcesCacheItem>& It)
		{
			return It.IsValid() && It->GetNativeResource() == InNativeResourcePtr;
		});

		if (Index != INDEX_NONE)
		{
			return Resources[Index].Get();
		}
	}

	return nullptr;
}

FTextureShareCoreResourcesCacheItem* FTextureShareCoreResourcesCache::FObjectCachedResources::FindByResourceDesc(const FTextureShareCoreResourceDesc& InResourceDesc)
{
	const int32 Index = Resources.IndexOfByPredicate([InResourceDesc](const TSharedPtr<FTextureShareCoreResourcesCacheItem>& It)
		{
			return It.IsValid() && It->GetHandle().ResourceDesc.EqualsFunc(InResourceDesc);
		});

	if (Index != INDEX_NONE)
	{
		return Resources[Index].Get();
	}

	return nullptr;
}

bool FTextureShareCoreResourcesCache::FObjectCachedResources::RemoveByNativeResourcePtr(const void* InNativeResourcePtr)
{
	bool bResult = false;

	for (TSharedPtr<FTextureShareCoreResourcesCacheItem>& ItemIt : Resources)
	{
		if (ItemIt.IsValid() && ItemIt->GetNativeResource() == InNativeResourcePtr)
		{
			ItemIt.Reset();
			bResult = true;
		}
	}

	Resources.Remove(nullptr);

	return bResult;
}

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreResourcesCache
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareCoreResourcesCache::FObjectCachedResources& FTextureShareCoreResourcesCache::GetObjectCachedResources(const FTextureShareCoreObjectDesc& InObjectDesc)
{
	FObjectCachedResources* const ExistObjectCache = Objects.Find(InObjectDesc.ObjectGuid);
	if (ExistObjectCache)
	{
		return *ExistObjectCache;
	}

	return Objects.Add(InObjectDesc.ObjectGuid);
}

bool FTextureShareCoreResourcesCache::RemoveObjectCachedResources(const FTextureShareCoreObjectDesc& InObjectDesc)
{
	FObjectCachedResources* const ExistObjectCache = Objects.Find(InObjectDesc.ObjectGuid);
	if (ExistObjectCache)
	{
		ExistObjectCache->Resources.Empty();
		Objects.Remove(InObjectDesc.ObjectGuid);

		return true;
	}

	return false;
}

void FTextureShareCoreResourcesCache::RemoveUnusedResources(const uint32 InMilisecondsTimeOut)
{
	for (TPair<FGuid, FObjectCachedResources> ObjectCachedResourcesIt : Objects)
	{
		for (TSharedPtr<FTextureShareCoreResourcesCacheItem>& ResourceIt : ObjectCachedResourcesIt.Value.Resources)
		{
			if (ResourceIt.IsValid())
			{
				if (ResourceIt->IsResourceUnused(InMilisecondsTimeOut))
				{
					// Release unused resource
					ResourceIt.Reset();
				}
			}
		}

		// Remove nullptr resources:
		ObjectCachedResourcesIt.Value.Resources.Remove(nullptr);
	}
}
