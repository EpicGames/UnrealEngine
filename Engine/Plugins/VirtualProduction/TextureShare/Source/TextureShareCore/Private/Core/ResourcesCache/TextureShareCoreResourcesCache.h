// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/TextureShareCoreContainers_DeviceVulkan.h"
#include "Core/ResourcesCache/TextureShareCoreResourcesCacheItem.h"

/**
 * Render resource cache base
 */
class FTextureShareCoreResourcesCache
{
public:
	virtual ~FTextureShareCoreResourcesCache() = default;

public:
	bool RemoveObjectCachedResources(const FTextureShareCoreObjectDesc& InObjectDesc);

	// Release all resources last used more than specified timeout value
	void RemoveUnusedResources(const uint32 InMilisecondsTimeOut);

protected:
	struct FObjectCachedResources
	{
		FTextureShareCoreResourcesCacheItem* Find(const FTextureShareCoreResourceHandle& InResourceHandle);
		FTextureShareCoreResourcesCacheItem* FindByNativeResourcePtr(const void* InNativeResourcePtr);
		FTextureShareCoreResourcesCacheItem* FindByResourceDesc(const FTextureShareCoreResourceDesc& InResourceDesc);

		bool RemoveByNativeResourcePtr(const void* InNativeResourcePtr);

	public:
		TArray<TSharedPtr<FTextureShareCoreResourcesCacheItem>> Resources;
	};

protected:
	FObjectCachedResources& GetObjectCachedResources(const FTextureShareCoreObjectDesc& InObjectDesc);

private:
	TMap<FGuid, FObjectCachedResources> Objects;
};
