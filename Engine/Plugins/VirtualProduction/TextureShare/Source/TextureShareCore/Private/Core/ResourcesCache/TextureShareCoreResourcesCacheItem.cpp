// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/ResourcesCache/TextureShareCoreResourcesCacheItem.h"
#include "Core/TextureShareCoreTime.h"
#include "Module/TextureShareCoreLog.h"

#if PLATFORM_WINDOWS
THIRD_PARTY_INCLUDES_START
#include "Windows/AllowWindowsPlatformTypes.h"
#include <handleapi.h>
#include "Windows/HideWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_END
#endif

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreResourcesCacheItem
//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreResourcesCacheItem::ReleaseHandle()
{
	// Release handles:
	if (bNeedReleaseNTHandle && Handle.NTHandle)
	{
		CloseHandle(Handle.NTHandle);
		Handle.NTHandle = nullptr;
	}
}

void FTextureShareCoreResourcesCacheItem::UpdateLastAccessTime()
{
	LastAccessTime = FTextureShareCoreTime::Cycles64();
}

bool FTextureShareCoreResourcesCacheItem::IsResourceUnused(const uint32 InMilisecondsTimeOut) const
{
	const uint64 ElaspsedCycle64    = FTextureShareCoreTime::Cycles64() - LastAccessTime;
	const uint32 ElapsedMiliseconds = FTextureShareCoreTime::Cycles64ToMiliseconds(ElaspsedCycle64);

	return ElapsedMiliseconds > InMilisecondsTimeOut;
}
