// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/TextureShareCoreContainers.h"
#include "Containers/TextureShareCoreContainers_DeviceVulkan.h"

/**
 * Render resource cache item base
 */
class FTextureShareCoreResourcesCacheItem
{
public:
	FTextureShareCoreResourcesCacheItem()
	{
		UpdateLastAccessTime();
	}

	virtual ~FTextureShareCoreResourcesCacheItem()
	{
		ReleaseHandle();
	}

	virtual void* GetNativeResource() const
	{
		return nullptr;
	}

public:
	const FTextureShareCoreResourceHandle& GetHandle() const
	{
		return Handle;
	}

	void SetResourceHandleGPUIndex(const uint32 InGPUIndex)
	{
		Handle.GPUIndex = InGPUIndex;
	}

	bool HandleEquals(const FTextureShareCoreResourceHandle& InResourceHandle) const
	{
		return Handle.HandleEquals(InResourceHandle);
	}

	void UpdateLastAccessTime();

	bool IsResourceUnused(const uint32 InMilisecondsTimeOut) const;

private:
	void ReleaseHandle();

protected:
	// Shared resource handles info
	FTextureShareCoreResourceHandle Handle;

	// garbage collector for unused resources
	uint64 LastAccessTime = 0;

	// Close handles in destructor
	bool bNeedReleaseNTHandle = false;
};
