// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/ResourcesCache/Vulkan/TextureShareCoreVulkanResourcesCache.h"
#include "Core/ResourcesCache/Vulkan/TextureShareCoreVulkanResourcesCacheItem.h"
#include "Core/TextureShareCoreSecurityAttributes.h"

#include "Module/TextureShareCoreLog.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreVulkanResourcesCache
//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreVulkanResourcesCache::CreateSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, const FTextureShareDeviceVulkanContext& InDeviceVulkanContext, const FTextureShareDeviceVulkanResource& InVulkanResource, const FTextureShareCoreResourceDesc& InResourceDesc, FTextureShareCoreResourceHandle& OutResourceHandle)
{
#if TEXTURESHARECORE_VULKAN
	if (InDeviceVulkanContext.IsValid() && InVulkanResource.IsValid())
	{
		FObjectCachedResources& ObjectResources = GetObjectCachedResources(InObjectDesc);

		// Use exist resource from cache
		if (FTextureShareCoreResourcesCacheItem* ExistResourcesCacheItem = ObjectResources.FindByNativeResourcePtr(InVulkanResource.GetNativeResourcePtr()))
		{
			OutResourceHandle = ExistResourcesCacheItem->GetHandle();

			check(OutResourceHandle.EqualsFunc(InResourceDesc));

			return true;
		}

		// Create new resource
		TSharedPtr<FTextureShareCoreResourcesCacheItem> NewResourcesCacheItem = MakeShared<FTextureShareCoreVulkanResourcesCacheItem>(InDeviceVulkanContext, InVulkanResource, InResourceDesc, SecurityAttributes->GetResourceSecurityAttributes());
		if (NewResourcesCacheItem.IsValid() && NewResourcesCacheItem->GetNativeResource())
		{
			ObjectResources.Resources.Add(NewResourcesCacheItem);
			OutResourceHandle = NewResourcesCacheItem->GetHandle();

			return true;
		}
	}
#endif

	return false;
}

FTextureShareDeviceVulkanResource FTextureShareCoreVulkanResourcesCache::OpenSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, const FTextureShareDeviceVulkanContext& InDeviceVulkanContext, const FTextureShareCoreResourceHandle& InResourceHandle)
{
#if TEXTURESHARECORE_VULKAN
	if (InDeviceVulkanContext.IsValid())
	{
		FObjectCachedResources& ObjectResources = GetObjectCachedResources(InObjectDesc);

		// Use exist resource from cache
		if (FTextureShareCoreResourcesCacheItem* ExistResourcesCacheItem = ObjectResources.Find(InResourceHandle))
		{
			if (FTextureShareCoreVulkanResourcesCacheItem* VulkanResource = static_cast<FTextureShareCoreVulkanResourcesCacheItem*>(ExistResourcesCacheItem))
			{
				return VulkanResource->VulkanResource;
			}
		}

		// Create new resource
		TSharedPtr<FTextureShareCoreVulkanResourcesCacheItem> NewResourcesCacheItem = MakeShared<FTextureShareCoreVulkanResourcesCacheItem>(InDeviceVulkanContext, InResourceHandle);
		if (NewResourcesCacheItem.IsValid() && NewResourcesCacheItem->GetNativeResource())
		{
			ObjectResources.Resources.Add(NewResourcesCacheItem);

			return NewResourcesCacheItem->VulkanResource;
		}
	}
#endif

	return FTextureShareDeviceVulkanResource();
}

bool FTextureShareCoreVulkanResourcesCache::RemoveSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, const FTextureShareDeviceVulkanContext& InDeviceVulkanContext, const FTextureShareDeviceVulkanResource& InVulkanResource)
{
#if TEXTURESHARECORE_VULKAN
	if (InVulkanResource.IsValid())
	{
		return GetObjectCachedResources(InObjectDesc).RemoveByNativeResourcePtr(InVulkanResource.GetNativeResourcePtr());
	}
#endif

	return false;
}
