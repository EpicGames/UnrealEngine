// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ITextureShareCoreVulkanResourcesCache.h"
#include "Core/ResourcesCache/TextureShareCoreResourcesCache.h"

class FTextureShareCoreSecurityAttributes;

/**
 * Vulkan renderer resources cache container
 */
class FTextureShareCoreVulkanResourcesCache
	: public ITextureShareCoreVulkanResourcesCache
	, public FTextureShareCoreResourcesCache
{
public:
	FTextureShareCoreVulkanResourcesCache(const TSharedPtr<FTextureShareCoreSecurityAttributes>& InSecurityAttributes)
		: SecurityAttributes(InSecurityAttributes)
	{ }
	virtual ~FTextureShareCoreVulkanResourcesCache() = default;

public:
	virtual bool CreateSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, const FTextureShareDeviceVulkanContext& InDeviceVulkanContext, const FTextureShareDeviceVulkanResource& InVulkanResource, const FTextureShareCoreResourceDesc& InResourceDesc, FTextureShareCoreResourceHandle& OutResourceHandle) override;
	virtual FTextureShareDeviceVulkanResource OpenSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, const FTextureShareDeviceVulkanContext& InDeviceVulkanContext, const FTextureShareCoreResourceHandle& InResourceHandle) override;
	virtual bool RemoveSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, const FTextureShareDeviceVulkanContext& InDeviceVulkanContext, const FTextureShareDeviceVulkanResource& InVulkanResource) override;

private:
	TSharedPtr<FTextureShareCoreSecurityAttributes> SecurityAttributes;
};
