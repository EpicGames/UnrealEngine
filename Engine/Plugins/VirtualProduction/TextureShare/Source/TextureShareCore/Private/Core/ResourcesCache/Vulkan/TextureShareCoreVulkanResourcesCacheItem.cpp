// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/ResourcesCache/Vulkan/TextureShareCoreVulkanResourcesCacheItem.h"
#include "Module/TextureShareCoreLog.h"

#if TEXTURESHARECORE_VULKAN

#if PLATFORM_WINDOWS
THIRD_PARTY_INCLUDES_START
#include "Windows/AllowWindowsPlatformTypes.h"

#include "vulkan/vulkan_core.h"
#include "vulkan/vulkan_win32.h"

#include "Windows/HideWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_END
#endif


#define VK_USE_PLATFORM_WIN32_KHR 1
#define VK_LIBRARY_NAME "vulkan-1"
#define VK_GETFUNC(FuncId)\
	FuncId = (PFN_##FuncId)(::GetProcAddress((Windows::HMODULE)VulkanLibraryPtr, "vkGetInstanceProcAddr"));

#define GetVulkanDeviceProc(name)\
	PFN_vk##name name = (PFN_vk##name)GetVulkanLibrary().GetDeviceInstanceProcAddr(InDeviceVulkanContext, "vk"#name)

namespace UE::TextureShareCore::VulkanResourcesCacheItem
{
	//////////////////////////////////////////////////////////////////////////////////////////////
	// FTextureShareCoreVulkanLibrary
	//////////////////////////////////////////////////////////////////////////////////////////////
	class FTextureShareCoreVulkanLibrary
	{
	public:
		FTextureShareCoreVulkanLibrary();
		~FTextureShareCoreVulkanLibrary();

	public:
		bool IsValid() const
		{
			return VulkanLibraryPtr != nullptr && vkGetInstanceProcAddr != nullptr;
		}

		void* GetInstanceProcAddr(const FTextureShareDeviceVulkanContext& InDeviceContext, LPCSTR InProcName) const;
		void* GetProcAddr(LPCSTR InProcName) const;
		void* GetDeviceInstanceProcAddr(const FTextureShareDeviceVulkanContext& InDeviceContext, LPCSTR InProcName) const;

	protected:
		// Load the library
		bool Load();
		// Unload the library
		void Unload();

	private:
		void* VulkanLibraryPtr = nullptr;
		PFN_vkGetInstanceProcAddr vkGetInstanceProcAddr = nullptr;
	};

	//////////////////////////////////////////////////////////////////////////////////////////////
	FTextureShareCoreVulkanLibrary::FTextureShareCoreVulkanLibrary()
	{
		if (!Load())
		{
			Unload();
		}
	}

	FTextureShareCoreVulkanLibrary::~FTextureShareCoreVulkanLibrary()
	{
		Unload();
	}

	void* FTextureShareCoreVulkanLibrary::GetInstanceProcAddr(const FTextureShareDeviceVulkanContext& InDeviceContext, LPCSTR InProcName) const
	{
		if (IsValid() && InProcName && *InProcName)
		{
			return vkGetInstanceProcAddr(InDeviceContext.GetInstance(), InProcName);
		}

		return nullptr;
	}

	void* FTextureShareCoreVulkanLibrary::GetProcAddr(LPCSTR InProcName) const
	{
		if (IsValid() && InProcName && *InProcName)
		{
			return vkGetInstanceProcAddr(nullptr, InProcName);
		}

		return nullptr;
	}

	void* FTextureShareCoreVulkanLibrary::GetDeviceInstanceProcAddr(const FTextureShareDeviceVulkanContext& InDeviceContext, LPCSTR InProcName) const
	{
		const PFN_vkGetDeviceProcAddr GetDeviceProcAddr = (PFN_vkGetDeviceProcAddr)GetInstanceProcAddr(InDeviceContext, "vkGetDeviceProcAddr");
		if (GetDeviceProcAddr)
		{
			return GetDeviceProcAddr(InDeviceContext.GetDevice(), InProcName);
		}

		return nullptr;
	}

	bool FTextureShareCoreVulkanLibrary::Load()
	{
		VulkanLibraryPtr = ::LoadLibraryA(VK_LIBRARY_NAME);
		if (VulkanLibraryPtr)
		{
			vkGetInstanceProcAddr = (PFN_vkGetInstanceProcAddr)(::GetProcAddress((Windows::HMODULE)VulkanLibraryPtr, "vkGetInstanceProcAddr"));

			if (!vkGetInstanceProcAddr)
				return false;

			return true;
		}

		return false;
	}

	void FTextureShareCoreVulkanLibrary::Unload()
	{
		if (VulkanLibraryPtr)
		{
			Windows::FreeLibrary((Windows::HMODULE)VulkanLibraryPtr);
			VulkanLibraryPtr = nullptr;
		}
	}

	static FTextureShareCoreVulkanLibrary& GetVulkanLibrary()
	{
		static FTextureShareCoreVulkanLibrary VulkanLibrary;
		return VulkanLibrary;
	}

	static bool VulkanCreateSharedHandle(const FTextureShareDeviceVulkanContext& InDeviceVulkanContext, const FTextureShareDeviceVulkanResource& InVulkanResource, const bool bUseNTHandle, HANDLE& OutSharedHandle)
	{
		if (GetVulkanLibrary().IsValid())
		{
			if (InVulkanResource.IsValid() && InDeviceVulkanContext.IsValid())
			{
				// Generate VkMemoryGetWin32HandleInfoKHR
				VkMemoryGetWin32HandleInfoKHR MemoryGetHandleInfoKHR = {};

				MemoryGetHandleInfoKHR.sType = VK_STRUCTURE_TYPE_MEMORY_GET_WIN32_HANDLE_INFO_KHR;
				MemoryGetHandleInfoKHR.pNext = NULL;
				MemoryGetHandleInfoKHR.memory = InVulkanResource.GetAllocationHandle();
				MemoryGetHandleInfoKHR.handleType = bUseNTHandle ? VK_EXTERNAL_MEMORY_HANDLE_TYPE_OPAQUE_WIN32_BIT : VK_EXTERNAL_MEMORY_HANDLE_TYPE_OPAQUE_WIN32_KMT_BIT;

				if (GetVulkanDeviceProc(GetMemoryWin32HandleKHR))
				{
					const VkResult Result = GetMemoryWin32HandleKHR(InDeviceVulkanContext.GetDevice(), &MemoryGetHandleInfoKHR, &OutSharedHandle);
					if (Result == VK_SUCCESS)
					{
						return OutSharedHandle != 0 && OutSharedHandle != INVALID_HANDLE_VALUE;
					}
				}
			}
		}

		return false;
	}
};
using namespace UE::TextureShareCore::VulkanResourcesCacheItem;

#endif
//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreVulkanResourcesCacheItem
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareCoreVulkanResourcesCacheItem::~FTextureShareCoreVulkanResourcesCacheItem()
{
	if (bNeedReleaseVulkanResource && VulkanResource.IsValid())
	{
		// Not implemented
	}
}

// Create shared resource
FTextureShareCoreVulkanResourcesCacheItem::FTextureShareCoreVulkanResourcesCacheItem(const FTextureShareDeviceVulkanContext& InDeviceVulkanContext, const FTextureShareDeviceVulkanResource& InVulkanResource, const FTextureShareCoreResourceDesc& InResourceDesc, const void* InSecurityAttributes)
	: DeviceVulkanContext(InDeviceVulkanContext)
{

#if TEXTURESHARECORE_VULKAN
	if (InDeviceVulkanContext.IsValid() && InVulkanResource.IsValid())
	{
		{
			// Get Win32 Vulkan shared resources handle
			HANDLE SharedNTHandle;
			if (VulkanCreateSharedHandle(InDeviceVulkanContext, InVulkanResource, true, SharedNTHandle))
			{
				Handle.ResourceDesc = InResourceDesc;
				Handle.NTHandle = SharedNTHandle;
				Handle.SharedHandleGuid = FGuid();
				Handle.SharedHandle = nullptr;

				// Store valid texture ptr
				VulkanResource = InVulkanResource;

				return;
			}
		}
		{
			// Try not-NT handle:
			HANDLE SharedHandle;
			if (VulkanCreateSharedHandle(InDeviceVulkanContext, InVulkanResource, false, SharedHandle))
			{
				Handle.ResourceDesc = InResourceDesc;
				Handle.NTHandle = nullptr;
				Handle.SharedHandleGuid = FGuid();
				Handle.SharedHandle = SharedHandle;

				// Store valid texture ptr
				VulkanResource = InVulkanResource;

				return;
			}
		}
	}
#endif
}

// Open shared resource
FTextureShareCoreVulkanResourcesCacheItem::FTextureShareCoreVulkanResourcesCacheItem(const FTextureShareDeviceVulkanContext& InDeviceVulkanContext, const FTextureShareCoreResourceHandle& InResourceHandle)
	: DeviceVulkanContext(InDeviceVulkanContext)
{
#if TEXTURESHARECORE_VULKAN
	// Not implemented
#endif
}
