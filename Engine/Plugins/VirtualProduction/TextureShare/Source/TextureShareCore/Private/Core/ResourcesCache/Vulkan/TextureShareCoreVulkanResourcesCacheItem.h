// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Core/ResourcesCache/TextureShareCoreResourcesCacheItem.h"
#include "Containers/TextureShareCoreContainers_DeviceVulkan.h"

/**
 * Vulkan renderer resource container
 */
class FTextureShareCoreVulkanResourcesCacheItem
	: public FTextureShareCoreResourcesCacheItem
{
public:
	// Create shared resource
	FTextureShareCoreVulkanResourcesCacheItem(const FTextureShareDeviceVulkanContext& InDeviceVulkanContext, const FTextureShareDeviceVulkanResource& InVulkanResource, const FTextureShareCoreResourceDesc& InResourceDesc, const void* InSecurityAttributes);

	// Open shared resource
	FTextureShareCoreVulkanResourcesCacheItem(const FTextureShareDeviceVulkanContext& InDeviceVulkanContext, const FTextureShareCoreResourceHandle& InResourceHandle);

	virtual ~FTextureShareCoreVulkanResourcesCacheItem();

	virtual void* GetNativeResource() const override
	{
		return VulkanResource.GetNativeResourcePtr();
	}

public:
	const FTextureShareDeviceVulkanContext DeviceVulkanContext;

	// Opened from Handle resources released
	bool bNeedReleaseVulkanResource = false;
	FTextureShareDeviceVulkanResource VulkanResource;
};
