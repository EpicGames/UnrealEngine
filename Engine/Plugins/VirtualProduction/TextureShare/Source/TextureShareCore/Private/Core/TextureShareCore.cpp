// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/TextureShareCore.h"
#include "Core/TextureShareCoreHelpers.h"
#include "Core/TextureShareCoreTime.h"
#include "Core/TextureShareCoreSecurityAttributes.h"
#include "Core/ResourcesCache/D3D11/TextureShareCoreD3D11ResourcesCache.h"
#include "Core/ResourcesCache/D3D12/TextureShareCoreD3D12ResourcesCache.h"
#include "Core/ResourcesCache/Vulkan/TextureShareCoreVulkanResourcesCache.h"

#include "Misc/TextureShareCoreStrings.h"
#include "Module/TextureShareCoreLog.h"
#include "Object/TextureShareCoreObject.h"

#include "Containers/TextureShareCoreContainers.h"

#include "IPC/TextureShareCoreInterprocessEvent.h"
#include "IPC/TextureShareCoreInterprocessMutex.h"
#include "IPC/TextureShareCoreInterprocessMemoryRegion.h"
#include "IPC/Containers/TextureShareCoreInterprocessMemory.h"

namespace UE::TextureShareCore
{
	static inline FString GetValidTextureShareProcessName(const FString& InProcessName)
	{
		if (InProcessName.IsEmpty())
		{
#if TEXTURESHARECORE_SDK
			return UE::TextureShareCoreStrings::DefaultProcessName::SDK;
#else
			return UE::TextureShareCoreStrings::DefaultProcessName::UE;
#endif
		};

		return InProcessName;
	}
};
using namespace UE::TextureShareCore;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCore
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareCore::FTextureShareCore()
{
	SecurityAttributes = MakeShared<FTextureShareCoreSecurityAttributes>();
	FTextureShareCoreTime::InitTiming();
}

FTextureShareCore::~FTextureShareCore()
{
	EndSession();

	SecurityAttributes.Reset();
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCore::BeginSession()
{
	check(IsInGameThread());
	check(SecurityAttributes.IsValid());

	if (!bIsSessionActive && InitializeInterprocessMemory())
	{
#if TEXTURESHARECORE_SDK
		// For the SDK, we always use this type of process.
		ProcessDesc.ProcessType = ETextureShareProcessType::SDK;
#else
		// For UnrealEngine, we use the default process type.
		ProcessDesc.ProcessType = ETextureShareProcessType::UE;
#endif

		// Update process name
		ProcessDesc.ProcessId = GetValidTextureShareProcessName(ProcessDesc.ProcessId);

		// Generate a new unique local process marker
		ProcessDesc.ProcessGuid = FGuid::NewGuid();

		// Create new resource caches
		D3D11ResourcesCache  = MakeShared<FTextureShareCoreD3D11ResourcesCache, ESPMode::ThreadSafe>(SecurityAttributes);
		D3D12ResourcesCache  = MakeShared<FTextureShareCoreD3D12ResourcesCache, ESPMode::ThreadSafe>(SecurityAttributes);
		VulkanResourcesCache = MakeShared<FTextureShareCoreVulkanResourcesCache, ESPMode::ThreadSafe>(SecurityAttributes);

		bIsSessionActive = true;

		UE_TS_LOG(LogTextureShareCore, Log, TEXT("Begin session '%s'"), *ProcessDesc.ProcessId);

		return true;
	}

	return false;
}

void FTextureShareCore::EndSession()
{
	check(IsInGameThread());

	if (bIsSessionActive)
	{
		// Release shared objects
		for (TPair<FString, TSharedPtr<FTextureShareCoreObject, ESPMode::ThreadSafe>>& It : Objects)
		{
			It.Value.Reset();
		}
		Objects.Empty();

		D3D11ResourcesCache.Reset();
		D3D12ResourcesCache.Reset();
		VulkanResourcesCache.Reset();

		ReleaseInterprocessMemory();

		// Reset this process
		bIsSessionActive = false;

		UE_TS_LOG(LogTextureShareCore, Log, TEXT("End session '%s'"), *ProcessDesc.ProcessId);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCore::IsActive() const
{
	return InterprocessMemoryRegion.IsValid() && InterprocessMemoryMutex.IsValid();
}

bool FTextureShareCore::IsSessionActive() const
{
	return IsActive() && bIsSessionActive;

}

//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<FEvent, ESPMode::ThreadSafe> FTextureShareCore::CreateInterprocessEvent(const FGuid& InEventGuid)
{
	check(SecurityAttributes.IsValid());

	return FTextureShareCoreInterprocessEventWin::CreateInterprocessEvent(InEventGuid, SecurityAttributes->GetSecurityAttributes(ETextureShareSecurityAttributesType::Event));
}

TSharedPtr<FEvent, ESPMode::ThreadSafe> FTextureShareCore::OpenInterprocessEvent(const FGuid& InEventGuid)
{
	check(SecurityAttributes.IsValid());

	return FTextureShareCoreInterprocessEventWin::OpenInterprocessEvent(InEventGuid, SecurityAttributes->GetSecurityAttributes(ETextureShareSecurityAttributesType::Event));
}

//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareCoreInterprocessMemory* FTextureShareCore::GetInterprocessMemory() const
{
	return IsSessionActive() ? InterprocessMemoryRegion->GetInterprocessMemory() : nullptr;
}

bool FTextureShareCore::LockInterprocessMemory(const uint32 MaxMillisecondsToWait) const
{
	// Isolate shared memory
	if (InterprocessMemoryMutex.IsValid())
	{
		if (InterprocessMemoryMutex->LockMutex(MaxMillisecondsToWait))
		{
			return true;
		}

		// Error. Process dead inside lock, so unlock and try again
		UE_LOGF(LogTextureShareCore, Error, "MemoryMutex deadlock. released");
		InterprocessMemoryMutex->UnlockMutex();

		if (InterprocessMemoryMutex->LockMutex(MaxMillisecondsToWait))
		{
			return true;
		}

		UE_LOGF(LogTextureShareCore, Error, "MemoryMutex deadlock#2 in-row. Disable mem access");

		return false;
	}

	return false;
}

void FTextureShareCore::UnlockInterprocessMemory() const
{
	if (InterprocessMemoryMutex.IsValid())
	{
		InterprocessMemoryMutex->UnlockMutex();
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCore::InitializeInterprocessMemory()
{
	// An open memory mutex that is used to synchronize all shared memory operations.
	if (!InterprocessMemoryMutex.IsValid())
	{
		InterprocessMemoryMutex = MakeShared<FTextureShareCoreInterprocessMutex, ESPMode::ThreadSafe>();
		if (!InterprocessMemoryMutex.IsValid() || !InterprocessMemoryMutex->InitializeInterprocessMutex(InterprocessMemoryMutexName))
		{
			UE_LOGF(LogTextureShareCore, Error, "Can't initialize process shared memory mutex");

			InterprocessMemoryMutex.Reset();

			return false;
		}
	}

	// Initialize shared memory
	if (!InterprocessMemoryRegion.IsValid())
	{
		InterprocessMemoryRegion = MakeShared<FTextureShareCoreInterprocessMemoryRegion, ESPMode::ThreadSafe>(InterprocessMemoryName);
		check(InterprocessMemoryRegion.IsValid());

		// Lock the memory mutex to gain exclusive access to the IPC memory handle.
		// Note: This lock helps to eliminate the use of a shared memory handle by multiple processes at the same time.
		//       (This happens when the IPC memory tries to be created or opened depending on the state of its handle).
		//       For example, without this lock, we can catch a rare glitch when interprocess memory is opened twice, and both processes access different areas of memory.
		if (LockInterprocessMemory(500))
		{
			const bool bResult = InterprocessMemoryRegion->Initialize();

			UnlockInterprocessMemory();

			if (!bResult)
			{
				UE_LOGF(LogTextureShareCore, Error, "Can't initialize shared memory region");

			InterprocessMemoryRegion.Reset();
			InterprocessMemoryMutex.Reset();

			return false;
		}
	}

	return true;
}

	UE_LOGF(LogTextureShareCore, Error, "Can't lock process shared memory mutex");

	return false;
}

void FTextureShareCore::ReleaseInterprocessMemory()
{
	InterprocessMemoryMutex.Reset();
	InterprocessMemoryRegion.Reset();
}

//////////////////////////////////////////////////////////////////////////////////////////////
// ITextureShareCoreAPI
//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<ITextureShareCoreObject, ESPMode::ThreadSafe> FTextureShareCore::GetOrCreateCoreObject(const FString& InShareName, const ETextureShareProcessType InProcessType)
{
	if (bIsSessionActive)
	{
		if (InShareName.IsEmpty())
		{
			UE_LOGF(LogTextureShareCore, Error, "GetGetOrCreate: the name is empty");

			return nullptr;
		}
		const FString ShareNameLwr = InShareName.ToLower();

		// Get exist
		const TSharedPtr<FTextureShareCoreObject, ESPMode::ThreadSafe>* ShareItem = Objects.Find(ShareNameLwr);
		if (ShareItem && ShareItem->IsValid())
		{
			return *ShareItem;
		}

		// Create new
		TSharedPtr<FTextureShareCoreObject, ESPMode::ThreadSafe> NewTextureShareCoreObject = MakeShared<FTextureShareCoreObject, ESPMode::ThreadSafe>(*this, ShareNameLwr, InProcessType);

		// Is created resource valid (out of max nums, unsupported devices, etc)
		if (NewTextureShareCoreObject.IsValid() && NewTextureShareCoreObject->IsActive())
		{
			Objects.Add(ShareNameLwr, NewTextureShareCoreObject);
			UE_LOGF(LogTextureShareCore, Verbose, "Created object: '%ls'", *InShareName);

			return NewTextureShareCoreObject;
		}

		NewTextureShareCoreObject.Reset();
		UE_LOGF(LogTextureShareCore, Error, "Create Object: the object with name '%ls' not created", *InShareName);
	}

	return nullptr;
}

bool FTextureShareCore::IsCoreObjectExist(const FString& ShareName) const
{
	return !ShareName.IsEmpty() && Objects.Contains(ShareName.ToLower());
}

bool FTextureShareCore::RemoveCoreObject(const FString& InShareName)
{
	if (!InShareName.IsEmpty() && Objects.Remove(InShareName.ToLower()) > 0)
	{
		UE_LOGF(LogTextureShareCore, Verbose, "Removed object '%ls'", *InShareName);

		return true;
	}

	return false;
}

//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<ITextureShareCoreD3D11ResourcesCache, ESPMode::ThreadSafe> FTextureShareCore::GetD3D11ResourcesCache()
{
	return D3D11ResourcesCache;
}

TSharedPtr<ITextureShareCoreD3D12ResourcesCache, ESPMode::ThreadSafe> FTextureShareCore::GetD3D12ResourcesCache()
{
	return D3D12ResourcesCache;
}

TSharedPtr<ITextureShareCoreVulkanResourcesCache, ESPMode::ThreadSafe> FTextureShareCore::GetVulkanResourcesCache()
{
	return VulkanResourcesCache;
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCore::RemoveCachedResources(const FTextureShareCoreObjectDesc& InObjectDesc)
{
	bool bResult = false;

	if (D3D11ResourcesCache.IsValid())
	{
		if (D3D11ResourcesCache->RemoveObjectCachedResources(InObjectDesc))
		{
			bResult = true;
		}
	}

	if (D3D12ResourcesCache.IsValid())
	{
		if (D3D12ResourcesCache->RemoveObjectCachedResources(InObjectDesc))
		{
			bResult = true;
		}
	}

	if (VulkanResourcesCache.IsValid())
	{
		if (VulkanResourcesCache->RemoveObjectCachedResources(InObjectDesc))
		{
			bResult = true;
		}
	}

	return bResult;
}

void FTextureShareCore::RemoveUnusedResources(const uint32 InMilisecondsTimeOut)
{
	if (D3D11ResourcesCache.IsValid())
	{
		D3D11ResourcesCache->RemoveUnusedResources(InMilisecondsTimeOut);
	}

	if (D3D12ResourcesCache.IsValid())
	{
		D3D12ResourcesCache->RemoveUnusedResources(InMilisecondsTimeOut);
	}

	if (VulkanResourcesCache.IsValid())
	{
		VulkanResourcesCache->RemoveUnusedResources(InMilisecondsTimeOut);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCore::GetInterprocessObjects(const FString& InShareName, TArraySerializable<FTextureShareCoreObjectDesc>& OutInterprocessObjects) const
{
	bool bResult = false;
	
	if(LockInterprocessMemory(100))
	{
		if (FTextureShareCoreInterprocessMemory* InterprocessMemory = GetInterprocessMemory())
		{
			if (InShareName.IsEmpty())
			{
				bResult = InterprocessMemory->FindInterprocessObjects(OutInterprocessObjects) > 0;
			}
			else
			{
				bResult = InterprocessMemory->FindInterprocessObjects(OutInterprocessObjects, InShareName) > 0;
			}
		}

		UnlockInterprocessMemory();
	}

	return bResult;
}

const FTextureShareCoreObjectProcessDesc& FTextureShareCore::GetProcessDesc() const
{
	return ProcessDesc;
}

void FTextureShareCore::SetProcessName(const FString& InProcessName)
{
	check(IsInGameThread());

	ProcessDesc.ProcessId = GetValidTextureShareProcessName(InProcessName);

	UE_TS_LOG(LogTextureShareCore, Log, TEXT("Set global process name '%s'"), *InProcessName);
}

bool FTextureShareCore::SetProcessDeviceType(const ETextureShareDeviceType InDeviceType)
{
	check(IsInGameThread());

	ProcessDesc.DeviceType = InDeviceType;

	UE_TS_LOG(LogTextureShareCore, Log, TEXT("Set global device type to '%s'"), GetTEXT(InDeviceType));

	return true;
}
