// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ITextureShareCoreAPI.h"
#include "TextureShareCoreCallbacks.h"

#include "Containers/TextureShareCoreContainers.h"

class FTextureShareCoreObject;
class FTextureShareCoreInterprocessMemoryRegion;
class FTextureShareCoreInterprocessMutex;
class FTextureShareCoreD3D11ResourcesCache;
class FTextureShareCoreD3D12ResourcesCache;
class FTextureShareCoreVulkanResourcesCache;
class FTextureShareCoreSecurityAttributes;

/**
 * TextureShareCore API implementation
 */
class FTextureShareCore
	: public ITextureShareCoreAPI
{
public:
	FTextureShareCore();
	virtual ~FTextureShareCore();

public:
	bool BeginSession();
	void EndSession();

public:
	//~ITextureShareCoreAPI
	virtual TSharedPtr<class ITextureShareCoreObject, ESPMode::ThreadSafe> GetOrCreateCoreObject(const FString& ShareName, const ETextureShareProcessType InProcessType = ETextureShareProcessType::Undefined) override;
	virtual bool IsCoreObjectExist(const FString& ShareName) const override;
	virtual bool RemoveCoreObject(const FString& ShareName) override;

	virtual TSharedPtr<class ITextureShareCoreD3D11ResourcesCache, ESPMode::ThreadSafe>  GetD3D11ResourcesCache() override;
	virtual TSharedPtr<class ITextureShareCoreD3D12ResourcesCache, ESPMode::ThreadSafe>  GetD3D12ResourcesCache() override;
	virtual TSharedPtr<class ITextureShareCoreVulkanResourcesCache, ESPMode::ThreadSafe> GetVulkanResourcesCache() override;

	virtual void RemoveUnusedResources(const uint32 InMilisecondsTimeOut) override;

	virtual bool GetInterprocessObjects(const FString& InShareName, TArraySerializable<FTextureShareCoreObjectDesc>& OutInterprocessObjects) const override;

	virtual const FTextureShareCoreObjectProcessDesc& GetProcessDesc() const override;

	virtual void SetProcessName(const FString& InProcessId) override;
	virtual bool SetProcessDeviceType(const ETextureShareDeviceType InDeviceType) override;

	virtual ITextureShareCoreCallbacks& GetCallbacks() override
	{
		return Callbacks;
	}
	//~~ITextureShareCoreAPI

	bool RemoveCachedResources(const FTextureShareCoreObjectDesc& ObjectDesc);

public:
	bool IsActive() const;
	bool IsSessionActive() const;

public:
	struct FTextureShareCoreInterprocessMemory* GetInterprocessMemory() const;

	bool LockInterprocessMemory(const uint32 MaxMillisecondsToWait) const;
	void UnlockInterprocessMemory() const;

	TSharedPtr<FEvent, ESPMode::ThreadSafe> CreateInterprocessEvent(const FGuid& InEventGuid);
	TSharedPtr<FEvent, ESPMode::ThreadSafe> OpenInterprocessEvent(const FGuid& InEventGuid);

private:
	bool InitializeInterprocessMemory();
	void ReleaseInterprocessMemory();

private:
	// Interprocess shared memory uses predefined names
	static constexpr auto InterprocessMemoryName = TEXT("{DA90FCAF-7034-4FF3-A3CE-C1FCEB12036B}");
	static constexpr auto InterprocessMemoryMutexName = TEXT("TextureShare_MemoryMutex");

	bool bIsSessionActive = false;

	TSharedPtr<FTextureShareCoreInterprocessMemoryRegion, ESPMode::ThreadSafe> InterprocessMemoryRegion;
	TSharedPtr<FTextureShareCoreInterprocessMutex, ESPMode::ThreadSafe>        InterprocessMemoryMutex;

	TSharedPtr<FTextureShareCoreD3D11ResourcesCache, ESPMode::ThreadSafe>  D3D11ResourcesCache;
	TSharedPtr<FTextureShareCoreD3D12ResourcesCache, ESPMode::ThreadSafe>  D3D12ResourcesCache;
	TSharedPtr<FTextureShareCoreVulkanResourcesCache, ESPMode::ThreadSafe> VulkanResourcesCache;

	TSharedPtr<FTextureShareCoreSecurityAttributes> SecurityAttributes;

	// Local process description
	FTextureShareCoreObjectProcessDesc ProcessDesc;

	// TS Objects
	TMap<FString, TSharedPtr<FTextureShareCoreObject, ESPMode::ThreadSafe>> Objects;

	FTextureShareCoreCallbacks Callbacks;
};
