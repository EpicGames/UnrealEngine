// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/TextureShareCoreCallbacks.h"

#include "ITextureShareCore.h"
#include "ITextureShareCoreAPI.h"
#include "ITextureShareCoreCallbacks.h"

ITextureShareCoreCallbacks& ITextureShareCoreCallbacks::Get()
{
	static ITextureShareCoreCallbacks& TSCallbacksAPI = ITextureShareCore::Get().GetTextureShareCoreAPI().GetCallbacks();

	return TSCallbacksAPI;
}
