// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITextureShareCoreCallbacks.h"

/**
 * TextureShareCore callbacks API implementation
 */
class FTextureShareCoreCallbacks :
	public  ITextureShareCoreCallbacks
{
public:
	virtual ~FTextureShareCoreCallbacks() = default;

public:
	virtual FTextureShareCoreBeginSessionEvent& OnTextureShareCoreBeginSession() override
	{
		return TextureShareCoreBeginSessionEvent;
	}

	virtual FTextureShareCoreEndSessionEvent& OnTextureShareCoreEndSession() override
	{
		return TextureShareCoreEndSessionEvent;
	}

	virtual FTextureShareCoreBeginFrameSyncEvent& OnTextureShareCoreBeginFrameSync() override
	{
		return TextureShareCoreBeginFrameSyncEvent;
	}

	virtual FTextureShareCoreEndFrameSyncEvent& OnTextureShareCoreEndFrameSync() override
	{
		return TextureShareCoreEndFrameSyncEvent;
	}

	virtual FTextureShareCoreFrameSyncEvent& OnTextureShareCoreFrameSync() override
	{
		return TextureShareCoreFrameSyncEvent;
	}

	virtual FTextureShareCoreBeginFrameSyncEvent_RenderThread& OnTextureShareCoreBeginFrameSync_RenderThread() override
	{
		return TextureShareCoreBeginFrameSyncEvent_RenderThread;
	}

	virtual FTextureShareCoreEndFrameSyncEvent_RenderThread& OnTextureShareCoreEndFrameSync_RenderThread() override
	{
		return TextureShareCoreEndFrameSyncEvent_RenderThread;
	}

	virtual FTextureShareCoreFrameSyncEvent_RenderThread& OnTextureShareCoreFrameSync_RenderThread() override
	{
		return TextureShareCoreFrameSyncEvent_RenderThread;
	}

private:
	FTextureShareCoreBeginSessionEvent     TextureShareCoreBeginSessionEvent;
	FTextureShareCoreEndSessionEvent       TextureShareCoreEndSessionEvent;
	FTextureShareCoreBeginFrameSyncEvent   TextureShareCoreBeginFrameSyncEvent;
	FTextureShareCoreEndFrameSyncEvent     TextureShareCoreEndFrameSyncEvent;
	FTextureShareCoreFrameSyncEvent        TextureShareCoreFrameSyncEvent;

	FTextureShareCoreBeginFrameSyncEvent_RenderThread TextureShareCoreBeginFrameSyncEvent_RenderThread;
	FTextureShareCoreEndFrameSyncEvent_RenderThread   TextureShareCoreEndFrameSyncEvent_RenderThread;
	FTextureShareCoreFrameSyncEvent_RenderThread      TextureShareCoreFrameSyncEvent_RenderThread;
};
