// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IPC/Containers/TextureShareCoreInterprocessEnums.h"
#include "IPC/Containers/TextureShareCoreInterprocessObject.h"

#include "Containers/TextureShareCoreContainers.h"

/**
 * Enums to text wrappers for logging
 */
namespace UE::TextureShareCore
{
	// Debug log purpose
	static constexpr auto GetTEXT(const ETextureShareDeviceType In)
	{
		switch (In)
		{
		case ETextureShareDeviceType::D3D11:  return TEXT("D3D11");
		case ETextureShareDeviceType::D3D12:  return TEXT("D3D12");
		case ETextureShareDeviceType::Vulkan: return TEXT("Vulkan");
		default:
			break;
		}

		return TEXT("Undefined");
	}

	static constexpr auto GetTEXT(const ETextureShareFrameSyncTemplate In)
	{
		switch (In)
		{
		case ETextureShareFrameSyncTemplate::Default: return TEXT("Default");
		case ETextureShareFrameSyncTemplate::SDK:     return TEXT("SDK");
		case ETextureShareFrameSyncTemplate::DisplayCluster: return TEXT("DisplayCluster");
		default:
			break;
		}

		return TEXT("?");
	}

	static constexpr auto GetTEXT(const ETextureShareThreadMutex In)
	{
		switch (In)
		{
		case ETextureShareThreadMutex::GameThread:      return TEXT("GameThread");
		case ETextureShareThreadMutex::RenderingThread: return TEXT("RenderingThread");
		case ETextureShareThreadMutex::InternalLock:    return TEXT("InternalLock");

		case ETextureShareThreadMutex::InternalObjectDescMT:       return TEXT("InternalObjectDescMT");
		case ETextureShareThreadMutex::InternalFrameConnectionsMT: return TEXT("InternalFrameConnectionsMT");
		case ETextureShareThreadMutex::InternalSyncSettingsMT:     return TEXT("InternalSyncSettingsMT");
		default:
			break;
		}

		return TEXT("?");
	}

	static constexpr auto GetTEXT(const ETextureShareTextureOp In)
	{
		switch (In)
		{
		case ETextureShareTextureOp::Write:    return TEXT("Write");
		case ETextureShareTextureOp::Read:     return TEXT("Read");
		default:
			break;
		}

		return TEXT("?");
	}

	static constexpr auto GetTEXT(const ETextureShareEyeType InEyeType)
	{
		switch (InEyeType)
		{
		case ETextureShareEyeType::Default:     return TEXT("Default");
		case ETextureShareEyeType::StereoLeft:  return TEXT("StereoLeft");
		case ETextureShareEyeType::StereoRight: return TEXT("StereoRight");
		default:
			break;
		}

		return TEXT("?");
	}

	static constexpr auto GetTEXT(const ETextureShareSyncStep In)
	{
		switch (In)
		{
		case ETextureShareSyncStep::InterprocessConnection: return TEXT("IC");

			/**
				* Frame sync steps(GameThread)
				*/

		case ETextureShareSyncStep::FrameBegin: return TEXT("f");

		case ETextureShareSyncStep::FramePreSetupBegin: return TEXT("fpS");
		case ETextureShareSyncStep::FramePreSetupEnd: return TEXT("fpS+");

		case ETextureShareSyncStep::FrameSetupBegin: return TEXT("fS");
		case ETextureShareSyncStep::FrameSetupEnd: return TEXT("fS+");

		case ETextureShareSyncStep::FramePostSetupBegin: return TEXT("fPS");
		case ETextureShareSyncStep::FramePostSetupEnd: return TEXT("fPS+");

		case ETextureShareSyncStep::FrameFlush: return TEXT("flush");
		case ETextureShareSyncStep::FrameEnd: return TEXT("f+");

			/**
				* FrameProxy sync steps (RenderThread)
				*/

		case ETextureShareSyncStep::FrameProxyBegin: return TEXT("F");

		case ETextureShareSyncStep::FrameSceneFinalColorBegin: return TEXT("F_fc");
		case ETextureShareSyncStep::FrameSceneFinalColorEnd: return TEXT("F_fc+");

		case ETextureShareSyncStep::FrameProxyPreRenderBegin: return TEXT("FpR");
		case ETextureShareSyncStep::FrameProxyPreRenderEnd: return TEXT("FpR+");

		case ETextureShareSyncStep::FrameProxyRenderBegin: return TEXT("FR");
		case ETextureShareSyncStep::FrameProxyRenderEnd: return TEXT("FR+");

		case ETextureShareSyncStep::FrameProxyPostWarpBegin: return TEXT("FPW");
		case ETextureShareSyncStep::FrameProxyPostWarpEnd: return TEXT("FPW+");

		case ETextureShareSyncStep::FrameProxyPostRenderBegin: return TEXT("FPR");
		case ETextureShareSyncStep::FrameProxyPostRenderEnd: return TEXT("FPR+");

		case ETextureShareSyncStep::FrameProxyBackBufferReadyToPresentBegin: return TEXT("FBbPr");
		case ETextureShareSyncStep::FrameProxyBackBufferReadyToPresentEnd: return TEXT("FBbPr+");

		case ETextureShareSyncStep::FrameProxyFlush: return TEXT("FLUSH");
		case ETextureShareSyncStep::FrameProxyEnd: return TEXT("F+");

		default:
			break;
		}

		return TEXT("?");
	}

	static constexpr auto GetTEXT(const ETextureShareSyncPass In)
	{
		switch (In)
		{
		case ETextureShareSyncPass::Enter:    return TEXT("Enter");
		case ETextureShareSyncPass::Exit:     return TEXT("Exit");
		case ETextureShareSyncPass::Complete: return TEXT("Complete");
		default:
			break;
		}

		return TEXT("?");
	}

	static constexpr auto GetTEXT(const ETextureShareSyncState In)
	{
		switch (In)
		{
		case ETextureShareSyncState::Enter:          return TEXT("1");
		case ETextureShareSyncState::EnterCompleted: return TEXT("1+");

		case ETextureShareSyncState::Exit:          return TEXT("2");
		case ETextureShareSyncState::ExitCompleted: return TEXT("2+");

		case ETextureShareSyncState::Completed: return TEXT("##");

		case ETextureShareSyncState::Undefined:
			break;
		}

		return TEXT("?");
	}
	
	static FString ToString(const FTextureShareCoreObjectSyncState& InObjectSyncState)
	{
		return FString::Printf(TEXT("%s[%s] {%s < %s}"), GetTEXT(InObjectSyncState.Step), GetTEXT(InObjectSyncState.State), GetTEXT(InObjectSyncState.NextStep), GetTEXT(InObjectSyncState.PrevStep));
	}

	static FString ToString(const FTextureShareCoreObjectDesc& InObject)
	{
		return FString::Printf(TEXT("%s(%s)"), *InObject.ProcessDesc.ProcessId, *ToString(InObject.Sync.SyncState));
	}

	static FString ToString(const TArraySerializable<FTextureShareCoreObjectDesc>& InObjects)
	{
		FString Result;
		for (const FTextureShareCoreObjectDesc& ObjectDescIt : InObjects)
		{
			if (!Result.IsEmpty())
			{
				Result += TEXT(", ");
			}

			Result += ToString(ObjectDescIt);
		}

		return Result;
	}

	static FString ToString(const FTextureShareCoreViewDesc& InViewDesc)
	{
		return FString::Printf(TEXT("%s[%s]"), *InViewDesc.Id, GetTEXT(InViewDesc.EyeType));
	}

	static FString ToString(const FTextureShareCoreResourceDesc& InResourceDesc)
	{
		return FString::Printf(TEXT("%s.%s, Eye=%s, Op=%s, SyncStep=%s, SrcViewId='%s'"), *InResourceDesc.ViewDesc.Id, *InResourceDesc.ResourceName,
			GetTEXT(InResourceDesc.ViewDesc.EyeType),
			GetTEXT(InResourceDesc.OperationType), GetTEXT(InResourceDesc.SyncStep),
			*InResourceDesc.ViewDesc.SrcId
		);
	}

	static FString ToString(const FTextureShareCoreResourceRequest& InResourceRequest)
	{
		return FString::Printf(TEXT("%s, GPU#=%d"), *ToString(InResourceRequest.ResourceDesc), InResourceRequest.GPUIndex);
	}
};
