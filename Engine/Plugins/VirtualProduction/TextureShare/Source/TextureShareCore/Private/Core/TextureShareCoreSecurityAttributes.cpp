// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureShareCoreSecurityAttributes.h"
#include "Module/TextureShareCoreLog.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// Windows only include
//
#if PLATFORM_WINDOWS
THIRD_PARTY_INCLUDES_START
#include "Windows/AllowWindowsPlatformTypes.h"
#include <aclapi.h>
#include "Windows/HideWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_END
#endif

//////////////////////////////////////////////////////////////////////////////////////////////
class FWindowsSecurityAttributesHelper
{
public:
	FWindowsSecurityAttributesHelper(const ETextureShareSecurityAttributesType InType)
	{
		Initialize(InType);
	}

	~FWindowsSecurityAttributesHelper()
	{
		Release();
	}

	SECURITY_ATTRIBUTES* GetSecurityAttributes()
	{
		return bEnabled ? &SecurityAttributes : nullptr;
	}

protected:
	bool Initialize(const ETextureShareSecurityAttributesType InType);
	void Release();

private:
	PSID pEveryoneSID = nullptr;
	PSID pAdminSID = nullptr;
	PACL pACL = nullptr;

	PSECURITY_DESCRIPTOR SecurityDescriptorPtr = nullptr;
	SECURITY_ATTRIBUTES SecurityAttributes;

	bool bEnabled = false;
};


//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreD3DResource
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareCoreSecurityAttributes::FTextureShareCoreSecurityAttributes()
{
	for (int32 TypeIndex= 0; TypeIndex < (uint8)ETextureShareSecurityAttributesType::COUNT; TypeIndex++)
	{
		SecurityAttributes[TypeIndex] = MakeUnique<FWindowsSecurityAttributesHelper>((ETextureShareSecurityAttributesType)TypeIndex);
	}
}

FTextureShareCoreSecurityAttributes::~FTextureShareCoreSecurityAttributes()
{
	for (int32 TypeIndex = 0; TypeIndex < (uint8)ETextureShareSecurityAttributesType::COUNT; TypeIndex++)
	{
		SecurityAttributes[TypeIndex].Reset();
	}
}
const void* FTextureShareCoreSecurityAttributes::GetSecurityAttributes(const ETextureShareSecurityAttributesType InType) const
{
	if (InType < ETextureShareSecurityAttributesType::COUNT)
	{
		if (SecurityAttributes[(uint8)InType].IsValid())
		{
			SecurityAttributes[(uint8)InType]->GetSecurityAttributes();
		}
	}

	return nullptr;
}

//////////////////////////////////////////////////////////////////////////////////////////////
// FWindowsSecurityAttributesHelper
//////////////////////////////////////////////////////////////////////////////////////////////
bool FWindowsSecurityAttributesHelper::Initialize(const ETextureShareSecurityAttributesType InType)
{
	DWORD dwRes;

	EXPLICIT_ACCESS ExplicitAccess[2];
	SID_IDENTIFIER_AUTHORITY SIDAuthWorld = SECURITY_WORLD_SID_AUTHORITY;
	SID_IDENTIFIER_AUTHORITY SIDAuthNT = SECURITY_NT_AUTHORITY;

	DWORD InGRFAccessPermissions = KEY_ALL_ACCESS;

	switch (InType)
	{
	case ETextureShareSecurityAttributesType::Event:
		InGRFAccessPermissions = KEY_NOTIFY;
	default:
		break;
	}


	// Create a well-known SID for the Everyone group.
	if (!AllocateAndInitializeSid(&SIDAuthWorld, 1,
		SECURITY_WORLD_RID,
		0, 0, 0, 0, 0, 0, 0,
		&pEveryoneSID))
	{
		UE_LOGF(LogTextureShareCoreWindows, Error, "AllocateAndInitializeSid Error %u", GetLastError());
		Release();

		return false;
	}

	// Initialize an EXPLICIT_ACCESS structure for an ACE.
	// The ACE will allow Everyone read access to the key.
	ZeroMemory(&ExplicitAccess, 2 * sizeof(EXPLICIT_ACCESS));
	ExplicitAccess[0].grfAccessPermissions = InGRFAccessPermissions;
	ExplicitAccess[0].grfAccessMode = SET_ACCESS;
	ExplicitAccess[0].grfInheritance = NO_INHERITANCE;
	ExplicitAccess[0].Trustee.TrusteeForm = TRUSTEE_IS_SID;
	//ExplicitAccess[0].Trustee.TrusteeType = TRUSTEE_IS_WELL_KNOWN_GROUP;
	ExplicitAccess[0].Trustee.TrusteeType = TRUSTEE_IS_GROUP;
	ExplicitAccess[0].Trustee.ptstrName = (LPTSTR)pEveryoneSID;

	// Create a SID for the BUILTIN\Administrators group.
	if (!AllocateAndInitializeSid(&SIDAuthNT, 2,
		SECURITY_BUILTIN_DOMAIN_RID,
		DOMAIN_ALIAS_RID_ADMINS,
		0, 0, 0, 0, 0, 0,
		&pAdminSID))
	{
		UE_LOGF(LogTextureShareCoreWindows, Error, "AllocateAndInitializeSid Error %u", GetLastError());
		Release();

		return false;
	}

	// Initialize an EXPLICIT_ACCESS structure for an ACE.
	// The ACE will allow the Administrators group full access to
	// the key.
	ExplicitAccess[1].grfAccessPermissions = InGRFAccessPermissions;
	ExplicitAccess[1].grfAccessMode = SET_ACCESS;
	ExplicitAccess[1].grfInheritance = NO_INHERITANCE;
	ExplicitAccess[1].Trustee.TrusteeForm = TRUSTEE_IS_SID;
	ExplicitAccess[1].Trustee.TrusteeType = TRUSTEE_IS_GROUP;
	ExplicitAccess[1].Trustee.ptstrName = (LPTSTR)pAdminSID;

	// Create a new ACL that contains the new ACEs.
	dwRes = SetEntriesInAcl(2, ExplicitAccess, NULL, &pACL);
	if (ERROR_SUCCESS != dwRes)
	{
		UE_LOGF(LogTextureShareCoreWindows, Error, "SetEntriesInAcl Error %u", GetLastError());
		Release();

		return false;

	}

	// Initialize a security descriptor.  
	SecurityDescriptorPtr = (PSECURITY_DESCRIPTOR)LocalAlloc(LPTR, SECURITY_DESCRIPTOR_MIN_LENGTH);
	if (NULL == SecurityDescriptorPtr)
	{
		UE_LOGF(LogTextureShareCoreWindows, Error, "LocalAlloc Error %u", GetLastError());
		Release();

		return false;

	}

	if (!InitializeSecurityDescriptor(SecurityDescriptorPtr, SECURITY_DESCRIPTOR_REVISION))
	{
		UE_LOGF(LogTextureShareCoreWindows, Error, "InitializeSecurityDescriptor Error %u", GetLastError());
		Release();

		return false;
	}

	// Add the ACL to the security descriptor. 
	if (!SetSecurityDescriptorDacl(SecurityDescriptorPtr,
		true,     // bDaclPresent flag   
		pACL,
		false))   // not a default DACL 
	{
		UE_LOGF(LogTextureShareCoreWindows, Error, "SetSecurityDescriptorDacl Error %u", GetLastError());
		Release();

		return false;
	}

	// Initialize a security attributes structure.
	SecurityAttributes.nLength = sizeof(SECURITY_ATTRIBUTES);
	SecurityAttributes.lpSecurityDescriptor = SecurityDescriptorPtr;
	SecurityAttributes.bInheritHandle = false;

	bEnabled = true;

	return true;
}

void FWindowsSecurityAttributesHelper::Release()
{
	if (pEveryoneSID)
	{
		FreeSid(pEveryoneSID);
	}

	if (pAdminSID)
	{
		FreeSid(pAdminSID);
	}

	if (pACL)
	{
		LocalFree(pACL);
	}

	if (SecurityDescriptorPtr)
	{
		LocalFree(SecurityDescriptorPtr);
	}

	bEnabled = false;
}
