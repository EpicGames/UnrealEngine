// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

class FWindowsSecurityAttributesHelper;

enum class ETextureShareSecurityAttributesType : uint8
{
	Resource = 0,
	Event,

	COUNT
};

/**
 * Windows security descriptors for IPC
 */
class FTextureShareCoreSecurityAttributes
{
public:
	FTextureShareCoreSecurityAttributes();
	~FTextureShareCoreSecurityAttributes();

public:
	const void* GetSecurityAttributes(const ETextureShareSecurityAttributesType InType) const;

private:
	TUniquePtr<FWindowsSecurityAttributesHelper> SecurityAttributes[(uint8)ETextureShareSecurityAttributesType::COUNT];
};
