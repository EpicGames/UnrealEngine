// Copyright Epic Games, Inc. All Rights Reserved.

#include "Core/TextureShareCoreTime.h"
#include "Windows/WindowsHWrapper.h"

//////////////////////////////////////////////////////////////////////////////////////////////
uint64 FTextureShareCoreTime::InitCycles64 = 0;
uint64 FTextureShareCoreTime::Cycle64PerSecond = 0;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreTime
//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreTime::InitTiming()
{
	LARGE_INTEGER Frequency;
	verify(QueryPerformanceFrequency(&Frequency));

	Cycle64PerSecond = Frequency.QuadPart;
}
