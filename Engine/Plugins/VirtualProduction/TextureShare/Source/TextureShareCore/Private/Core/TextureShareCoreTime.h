// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Windows/WindowsSystemIncludes.h"

/**
 * Internal time API for TextureShare (SDK compiled without UE engine)
 */
struct FTextureShareCoreTime
{
	static void InitTiming();

	static uint64 Cycles64()
	{
		Windows::LARGE_INTEGER Cycles;
		Windows::QueryPerformanceCounter(&Cycles);
		return Cycles.QuadPart;
	}

	static uint32 Cycles64ToMiliseconds(const uint64 InCycles64)
	{
		return (InCycles64 * 1000) / Cycle64PerSecond;
	}

	static uint64 MilisecondsToCycles64(const uint32 InMiliseconds)
	{
		return (InMiliseconds * Cycle64PerSecond) / 1000;
	}

private:
	static uint64 InitCycles64;
	static uint64 Cycle64PerSecond;
};
