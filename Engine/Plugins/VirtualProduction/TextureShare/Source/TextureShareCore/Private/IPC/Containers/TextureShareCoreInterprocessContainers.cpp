// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPC/Containers/TextureShareCoreInterprocessContainers.h"

#include "Misc/SecureHash.h"
#include "Windows/MinimalWindowsApi.h"

namespace UE::TextureShareCore::InterprocessContainers
{
	static FTextureShareCoreSMD5Hash CreateEmptySMD5Hash()
	{
		FTextureShareCoreSMD5Hash Result;
		Result.Empty();

		return Result;
	}
};
using namespace UE::TextureShareCore::InterprocessContainers;

//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareCoreSMD5Hash FTextureShareCoreSMD5Hash::EmptyValue = CreateEmptySMD5Hash();

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreSMD5Hash
//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreSMD5Hash::Initialize(const FString& InText)
{
	Empty();

	// Generate MD5 Hash from TCHAR as binary data
	FMD5 Md5Gen;
	Md5Gen.Update((const uint8*)GetData(InText), GetNum(InText) * sizeof(TCHAR));
	Md5Gen.Final(MD5Digest);
}

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreSMD5HashList
//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreSMD5HashList::Initialize(const TArraySerializable<FString>& InStringList)
{
	Empty();

	int32 Index = 0;
	for (const FString& StringIt : InStringList)
	{
		if (Index < MaxStringsCnt)
		{
			List[Index].Initialize(StringIt);
		}
		else
		{
			break;
		}

		Index++;
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreStringHash
//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreStringHash::Initialize(const FString& InText)
{
	Empty();

	Hash.Initialize(InText);

	FTCHARToWChar WCharString(*InText);
	{
		const int32 StrLength = FMath::Min(WCharString.Length(), MaxStringLength - 1);

		// Reset and copy wchar string
		FPlatformMemory::Memset(&String[0], 0, sizeof(String));
		FPlatformMemory::Memcpy(&String[0], WCharString.Get(), sizeof(wchar_t) * StrLength);
	}
}
