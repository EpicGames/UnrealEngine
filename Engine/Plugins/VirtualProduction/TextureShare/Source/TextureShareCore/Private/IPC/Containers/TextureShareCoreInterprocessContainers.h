// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "IPC/Containers/TextureShareCoreInterprocessEnums.h"
#include "Containers/TextureShareCoreContainers.h"
#include "Core/TextureShareCoreTime.h"
#include "Serialize/TextureShareCoreSerialize.h"

/**
 * Timestump container
 * This structure fits the criteria of being POD ("Plain Old Data") for binary compatibility with direct memory access.
 */
struct FTextureShareCoreTimestump
{
	uint64 Time;

public:
	void Empty()
	{
		Time = 0;
	}

	bool IsEmpty() const
	{
		return Time == 0;
	}

	// UPdate time value do current time
	void Update()
	{
		Time = FTextureShareCoreTime::Cycles64();
	}

	// return miliseconds from last update
	uint32 GetElapsedMilisecond() const
	{
		return FTextureShareCoreTime::Cycles64ToMiliseconds(FTextureShareCoreTime::Cycles64() - Time);
	}
};

/**
 * FGuid components container
 * This structure fits the criteria of being POD ("Plain Old Data") for binary compatibility with direct memory access.
 */
struct FTextureShareCoreGuid
{
	uint32 A;
	uint32 B;
	uint32 C;
	uint32 D;

public:
	bool Equals(const FTextureShareCoreGuid& In) const
	{
		return A == In.A && B == In.B && C == In.C && D == In.D;
	}

	void Empty()
	{
		A = B = C = D = 0;
	}

	bool IsEmpty() const
	{
		return A == 0 || B == 0 || C == 0 || D == 0;
	}

	FGuid ToGuid() const
	{
		return FGuid(A, B, C, D);
	}

	void Initialize(const FGuid& InGuid)
	{
		A = InGuid.A;
		B = InGuid.B;
		C = InGuid.C;
		D = InGuid.D;
	}

	static FTextureShareCoreGuid Create(const FGuid& InGuid)
	{
		FTextureShareCoreGuid Result;
		Result.Initialize(InGuid);

		return Result;
	}
};

/**
 * String MD5 hash container
 * This structure fits the criteria of being POD ("Plain Old Data") for binary compatibility with direct memory access.
 */
struct FTextureShareCoreSMD5Hash
{
public:
	// MD5 digest is 16 bytes
	static constexpr auto MD5DigestLen = 16;

	// MD5 hash value
	uint8 MD5Digest[MD5DigestLen];

public:
	bool Equals(const FTextureShareCoreSMD5Hash& In) const
	{
		return FPlatformMemory::Memcmp(&MD5Digest[0], &In.MD5Digest[0], sizeof(MD5Digest)) == 0;
	}

	bool IsEnabled() const
	{
		return !Equals(EmptyValue);
	}

	void Empty()
	{
		FPlatformMemory::Memset(&MD5Digest[0], 0, sizeof(MD5Digest));
	}

	void Initialize(const FString& InText);

	static FTextureShareCoreSMD5Hash Create(const FString& InString)
	{
		FTextureShareCoreSMD5Hash StrHash;
		StrHash.Initialize(InString);

		return StrHash;
	}

public:
	static FTextureShareCoreSMD5Hash EmptyValue;
};

/**
 * Simple constant array of MD5 string hashes
 * This structure fits the criteria of being POD ("Plain Old Data") for binary compatibility with direct memory access.
 */
struct FTextureShareCoreSMD5HashList
{
	// Max count of elements in array
	// This array is currently used as the holder of the sync settings process names.
	static constexpr auto MaxStringsCnt = 16;

	// MD5 hash values
	FTextureShareCoreSMD5Hash List[MaxStringsCnt];

public:
	void Empty()
	{
		FPlatformMemory::Memset(&List[0], 0, sizeof(List));
	}

	bool IsEmpty() const
	{
		return List[0].IsEnabled() == false;
	}

	int32 Find(const FTextureShareCoreSMD5Hash& InHash) const
	{
		for (int32 Index = 0; Index < MaxStringsCnt; Index++)
		{
			if (List[Index].Equals(InHash))
			{
				return Index;
			}

			if (!List[Index].IsEnabled())
			{
				break;
			}
		}

		return INDEX_NONE;
	}

	void Initialize(const TArraySerializable<FString>& InStringList);

	static FTextureShareCoreSMD5HashList Create(const TArraySerializable<FString>& InStringList)
	{
		FTextureShareCoreSMD5HashList Result;
		Result.Initialize(InStringList);

		return Result;
	}
};

/**
 * Unique MD5 hash produced from string 
 * This structure fits the criteria of being POD ("Plain Old Data") for binary compatibility with direct memory access.
 */
struct FTextureShareCoreStringHash
{
	// Maximum string len
	// This string is currently used as the holder of process names and textureshare objects.
	static constexpr auto MaxStringLength = 128;

	// String value
	wchar_t String[MaxStringLength];

	// MD5 hash of the string before
	FTextureShareCoreSMD5Hash Hash;

public:
	bool Equals(const FTextureShareCoreStringHash& In) const
	{
		return Hash.Equals(In.Hash);
	}

	bool Equals(const FTextureShareCoreSMD5Hash& InHash) const
	{
		return Hash.Equals(InHash);
	}

	FString ToString() const
	{
		return FString(&String[0]);
	}

	void Empty()
	{
		Hash.Empty();

		FPlatformMemory::Memset(&String[0], 0, sizeof(wchar_t)* MaxStringLength);
	}

	void Initialize(const FString& InText);

	static FTextureShareCoreStringHash Create(const FString& InText)
	{
		FTextureShareCoreStringHash Result;
		Result.Initialize(InText);
		return Result;
	}
};
