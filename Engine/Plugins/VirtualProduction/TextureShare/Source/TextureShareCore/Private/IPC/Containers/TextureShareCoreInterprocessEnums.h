// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Containers/TextureShareCoreEnums.h"

/**
 * Type of serialized data in memory area
 */
enum class ETextureShareCoreInterprocessObjectDataType : uint8
{
	// Mem block not defined
	Undefined = 0,

	// data blocks by types
	Frame,
	FrameProxy
};

/**
 * Object synchronization state (internal use only)
 */
enum class ETextureShareCoreInterprocessObjectFrameSyncState: uint8
{
	Undefined = 0,
	FrameSyncLost,

	NewFrame,
	FrameConnected,
	FrameBegin,
	FrameEnd,

	FrameProxyBegin,
	FrameProxyEnd,
};

/**
 * Object barrier synchronization state (internal use only)
 */
enum class ETextureShareInterprocessObjectSyncBarrierState : uint8
{
	Undefined = 0,

	// this process does not use this barrier
	UnusedSyncStep,

	// Special logic for connection, because at this monent processes reconnected
	WaitConnection,
	AcceptConnection,

	Wait,
	Accept,

	ObjectLost,
	FrameSyncLost,

	InvalidLogic,
};
