// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPC/Containers/TextureShareCoreInterprocessMemory.h"
#include "IPC/TextureShareCoreInterprocessHelpers.h"
#include "Module/TextureShareCoreLog.h"

using namespace UE::TextureShareCore;

/////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreInterprocessMemory
/////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreInterprocessMemory::IsUsedSyncBarrierPass(const TArraySerializable<FTextureShareCoreObjectDesc>& ObjectsDesc, const ETextureShareSyncStep InSyncStep)
{
	for (const FTextureShareCoreObjectDesc& ObjectDescIt : ObjectsDesc)
	{
		if (const FTextureShareCoreInterprocessObject* InterprocessObject = FindObject(ObjectDescIt))
		{
			if (InterprocessObject->IsEnabled() && InterprocessObject->Sync.GetSyncSettings().IsStepEnabled(InSyncStep))
			{
				return true;
			}
		}
	}

	return false;
}

int32 FTextureShareCoreInterprocessMemory::FindConnectableObjects(TArraySerializable<FTextureShareCoreObjectDesc>& OutConnectableObjects, const FTextureShareCoreInterprocessObject& InObject) const
{
	int32 ReadyToConnectObjectsCount = 0;
	OutConnectableObjects.Reset();

	// Search key: TextureShare name
	for (int32 ObjectIndex = 0; ObjectIndex < MaxNumberOfInterprocessObject; ObjectIndex++)
	{
		const FTextureShareCoreInterprocessObject& ObjectIt = Objects[ObjectIndex];
		const bool bIsInObject = (&InObject) == (&ObjectIt);

		if (!bIsInObject && ObjectIt.Desc.IsConnectable(InObject.Desc))
		{
			// test vs rules:
			if (ObjectIt.Sync.GetSyncSettings().IsShouldBeConnected(ObjectIt.Desc.ProcessName, InObject.Desc.ProcessName ,InObject.Sync.GetSyncSettings()))
			{
				// collect only ready to connect objects
				FTextureShareCoreObjectDesc ObjectDesc;
				if (ObjectIt.GetDesc(ObjectDesc))
				{
					OutConnectableObjects.AddUnique(ObjectDesc);

					// Collect ready to connect objects 
					const ETextureShareInterprocessObjectSyncBarrierState BarrierState = ObjectIt.Sync.GetBarrierState(InObject.Sync);
					switch (BarrierState)
					{
					case ETextureShareInterprocessObjectSyncBarrierState::AcceptConnection:
						// Count total connectable objects
						ReadyToConnectObjectsCount++;
						break;

					case ETextureShareInterprocessObjectSyncBarrierState::WaitConnection:
					default:
						break;
					}
				}
			}
		}
	}

	return ReadyToConnectObjectsCount;
}

void FTextureShareCoreInterprocessMemory::UpdateFrameConnections(TArraySerializable<FTextureShareCoreObjectDesc>& InOutFrameConnections) const
{
	for (FTextureShareCoreObjectDesc& ObjectDescIt : InOutFrameConnections)
	{
		if (const FTextureShareCoreInterprocessObject* InterprocessObject = FindObject(ObjectDescIt))
		{
			// Update data from shared mem
			FTextureShareCoreObjectDesc ObjectDesc;
			if (InterprocessObject->GetDesc(ObjectDesc))
			{
				ObjectDescIt = ObjectDesc;
			}
		}
	}
}

ETextureShareInterprocessObjectSyncBarrierState FTextureShareCoreInterprocessMemory::GetBarrierState(const FTextureShareCoreInterprocessObject& InObject, const TArraySerializable<FTextureShareCoreObjectDesc>& InFrameConnections) const
{
	int32 TotalAcceptedCnt = 0;
	for (const FTextureShareCoreObjectDesc& ObjectDescIt : InFrameConnections)
	{
		const FTextureShareCoreInterprocessObject* InterprocessObject = FindObject(ObjectDescIt);

		// Object may be lost while syncing
		if (InterprocessObject == nullptr || InterprocessObject->IsEnabled() == false)
		{
			return ETextureShareInterprocessObjectSyncBarrierState::ObjectLost;
		}

		switch (const ETextureShareInterprocessObjectSyncBarrierState BarrierState = InterprocessObject->Sync.GetBarrierState(InObject.Sync))
		{
		case ETextureShareInterprocessObjectSyncBarrierState::Accept:
		case ETextureShareInterprocessObjectSyncBarrierState::UnusedSyncStep:
			// accept this process
			TotalAcceptedCnt++;
			break;
		case ETextureShareInterprocessObjectSyncBarrierState::Wait:
			// Wait for this process
			break;
		case ETextureShareInterprocessObjectSyncBarrierState::WaitConnection:
			if (InObject.Sync.SyncState.Step != ETextureShareSyncStep::InterprocessConnection)
			{
				// accept this process
				UE_TS_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:GetBarrierState: skip sync for process %s"), *InObject.Desc.ShareName.ToString(), *InterprocessObject->Desc.ProcessName.ToString());
				TotalAcceptedCnt++;
				break;
			}
			return BarrierState;

		default:
			// Any error break frame sync
			return BarrierState;
		}
	}

	return TotalAcceptedCnt == InFrameConnections.Num() ? ETextureShareInterprocessObjectSyncBarrierState::Accept : ETextureShareInterprocessObjectSyncBarrierState::Wait;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareCoreInterprocessObject* FTextureShareCoreInterprocessMemory::FindEmptyObject()
{
	for (int32 ObjectIndex = 0; ObjectIndex < MaxNumberOfInterprocessObject; ObjectIndex++)
	{
		if (Objects[ObjectIndex].Desc.IsEnabled() == false)
		{
			UE_TS_LOG(LogTextureShareCoreObjectSync, Log, TEXT("Found Empty Object slot %d"), ObjectIndex);

			return &Objects[ObjectIndex];
		}
	}

	return nullptr;
}

int32 FTextureShareCoreInterprocessMemory::FindInterprocessObjects(TArraySerializable<FTextureShareCoreObjectDesc>& OutInterprocessObjects) const
{
	for (int32 ObjectIndex = 0; ObjectIndex < MaxNumberOfInterprocessObject; ObjectIndex++)
	{
		const FTextureShareCoreInterprocessObject& Object = Objects[ObjectIndex];

		FTextureShareCoreObjectDesc ObjectDesc;
		if (Object.IsEnabled() && Object.GetDesc(ObjectDesc))
		{
			OutInterprocessObjects.AddUnique(ObjectDesc);
		}
	}

	return OutInterprocessObjects.Num();
}

const FTextureShareCoreInterprocessObject* FTextureShareCoreInterprocessMemory::FindObject(const FTextureShareCoreObjectDesc& InObjectDesc) const
{
	// Search key: Unique object Guid
	const FTextureShareCoreGuid InObjectGuid = FTextureShareCoreGuid::Create(InObjectDesc.ObjectGuid);

	for (int32 ObjectIndex = 0; ObjectIndex < MaxNumberOfInterprocessObject; ObjectIndex++)
	{
		if (Objects[ObjectIndex].Desc.Equals(InObjectGuid))
		{
			return &Objects[ObjectIndex];
		}
	}

	return nullptr;
}

FTextureShareCoreInterprocessObject* FTextureShareCoreInterprocessMemory::FindObject(const FTextureShareCoreObjectDesc& InObjectDesc)
{
	// Search key: Unique object Guid
	const FTextureShareCoreGuid InObjectGuid = FTextureShareCoreGuid::Create(InObjectDesc.ObjectGuid);

	for (int32 ObjectIndex = 0; ObjectIndex < MaxNumberOfInterprocessObject; ObjectIndex++)
	{
		if (Objects[ObjectIndex].Desc.Equals(InObjectGuid))
		{
			return &Objects[ObjectIndex];
		}
	}

	return nullptr;
}

int32  FTextureShareCoreInterprocessMemory::FindInterprocessObjects(TArraySerializable<FTextureShareCoreObjectDesc>& OutInterprocessObjects, const FTextureShareCoreSMD5Hash& InHash) const
{
	OutInterprocessObjects.Reset();

	// Search key: TextureShare name
	for (int32 ObjectIndex = 0; ObjectIndex < MaxNumberOfInterprocessObject; ObjectIndex++)
	{
		const FTextureShareCoreInterprocessObject& Object = Objects[ObjectIndex];

		FTextureShareCoreObjectDesc ObjectDesc;
		if (Object.IsEnabled() && Object.Desc.IsShareNameEquals(InHash) && Object.GetDesc(ObjectDesc))
		{
			OutInterprocessObjects.AddUnique(ObjectDesc);
		}
	}

	return OutInterprocessObjects.Num();
}

int32 FTextureShareCoreInterprocessMemory::FindObjectEventListeners(TArray<const FTextureShareCoreInterprocessObject*>& OutObjects, const FTextureShareCoreObjectDesc& InObjectDesc) const
{
	OutObjects.Reset();

	const FTextureShareCoreSMD5Hash   InHash = FTextureShareCoreSMD5Hash::Create(InObjectDesc.ShareName);
	const FTextureShareCoreGuid InObjectGuid = FTextureShareCoreGuid::Create(InObjectDesc.ObjectGuid);

	const ETextureShareProcessType ProcessType = InObjectDesc.ProcessDesc.ProcessType;

	// Search key: TextureShare name
	for (int32 ObjectIndex = 0; ObjectIndex < MaxNumberOfInterprocessObject; ObjectIndex++)
	{
		const FTextureShareCoreInterprocessObject& Object = Objects[ObjectIndex];
		if(!Object.Desc.Equals(InObjectGuid) && Object.Desc.IsConnectable(InHash, ProcessType))
		{
			OutObjects.Add(&Object);
		}
	}

	return OutObjects.Num();
}

void FTextureShareCoreInterprocessMemory::ReleaseDirtyObjects(const FTextureShareCoreObjectDesc& InObjectDesc)
{
	// Search key: Unique object name + process name
	FTextureShareCoreStringHash InShareNameHash = FTextureShareCoreStringHash::Create(InObjectDesc.ShareName);
	FTextureShareCoreStringHash InProcessNameHash = FTextureShareCoreStringHash::Create(InObjectDesc.ProcessDesc.ProcessId);

	TArray<const FTextureShareCoreInterprocessObject*> OutObjects;
	for (int32 ObjectIndex = 0; ObjectIndex < MaxNumberOfInterprocessObject; ObjectIndex++)
	{
		if (Objects[ObjectIndex].Desc.IsShareNameEquals(InShareNameHash)
			&& Objects[ObjectIndex].Desc.ProcessName.Equals(InProcessNameHash))
		{
			UE_TS_LOG(LogTextureShareCoreObjectSync, Log, TEXT("Release lost object at slot %d, Guid='%s'"), ObjectIndex, *Objects[ObjectIndex].Desc.ObjectGuid.ToGuid().ToString(EGuidFormats::DigitsWithHyphens));

			// Release lost objects
			Objects[ObjectIndex].Release();
		}
	}
}
