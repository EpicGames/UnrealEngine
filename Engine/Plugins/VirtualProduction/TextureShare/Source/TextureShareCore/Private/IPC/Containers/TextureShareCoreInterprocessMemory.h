// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "IPC/Containers/TextureShareCoreInterprocessObject.h"
#include "Serialize/TextureShareCoreSerialize.h"

/**
 * This structure represents the entire block of shared memory.
 * This structure fits the criteria of being POD ("Plain Old Data") for binary compatibility with direct memory access.
 */
struct FTextureShareCoreInterprocessMemory
{
	// Maximum number of TextureShare IPC objects at the same time
	static constexpr auto MaxNumberOfInterprocessObject = 256;

	// TextureShare IPC objects containers
	FTextureShareCoreInterprocessObject Objects[MaxNumberOfInterprocessObject];

public:
	// Find all not empty objects
	const FTextureShareCoreInterprocessObject* FindObject(const FTextureShareCoreObjectDesc& InObjectDesc) const;
	FTextureShareCoreInterprocessObject* FindObject(const FTextureShareCoreObjectDesc& InObjectDesc);

	FTextureShareCoreInterprocessObject* FindEmptyObject();

	void ReleaseDirtyObjects(const FTextureShareCoreObjectDesc& InObjectDesc);

public:
	bool IsUsedSyncBarrierPass(const TArraySerializable<FTextureShareCoreObjectDesc>& ObjectsDesc, const ETextureShareSyncStep InSyncStep);
	int32 FindConnectableObjects(TArraySerializable<FTextureShareCoreObjectDesc>& OutConnectableObjects, const FTextureShareCoreInterprocessObject& InObject) const;

	ETextureShareInterprocessObjectSyncBarrierState GetBarrierState(const FTextureShareCoreInterprocessObject& InObject, const TArraySerializable<FTextureShareCoreObjectDesc>& InFrameConnections) const;

	void UpdateFrameConnections(TArraySerializable<FTextureShareCoreObjectDesc>& InOutFrameConnections) const;

public:
	int32 FindInterprocessObjects(TArraySerializable<FTextureShareCoreObjectDesc>& OutInterprocessObjects) const;

	// Return all IPC objects with same name
	int32 FindInterprocessObjects(TArraySerializable<FTextureShareCoreObjectDesc>& OutInterprocessObjects, const FTextureShareCoreSMD5Hash& InHash) const;

	int32 FindInterprocessObjects(TArraySerializable<FTextureShareCoreObjectDesc>& OutInterprocessObjects, const FString& InShareName) const
	{
		return FindInterprocessObjects(OutInterprocessObjects, FTextureShareCoreSMD5Hash::Create(InShareName));
	}

	// return remote IPC who may listen this object
	int32 FindObjectEventListeners(TArray<const FTextureShareCoreInterprocessObject*>& OutObjects, const FTextureShareCoreObjectDesc& InObjectDesc) const;
};
