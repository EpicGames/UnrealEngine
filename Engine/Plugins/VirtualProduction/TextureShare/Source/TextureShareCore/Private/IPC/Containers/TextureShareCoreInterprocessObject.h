// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IPC/Containers/TextureShareCoreInterprocessObjectDesc.h"
#include "IPC/Containers/TextureShareCoreInterprocessObjectData.h"
#include "IPC/Containers/TextureShareCoreInterprocessObjectSync.h"

/**
 * This structure represents the data of a TextureShare object in shared memory.
 * This structure fits the criteria of being POD ("Plain Old Data") for binary compatibility with direct memory access.
 */
struct FTextureShareCoreInterprocessObject
{
	// Object ID data
	FTextureShareCoreInterprocessObjectDesc Desc;

	// Object sync info
	FTextureShareCoreInterprocessObjectSync Sync;

	// Object data for IPC exchanges
	FTextureShareCoreInterprocessObjectData Data;

public:
	bool IsEnabled() const
	{
		return Desc.IsEnabled() && Sync.IsEnabled();
	}

	bool GetDesc(FTextureShareCoreObjectDesc& OutDesc) const
	{
		return Desc.GetDesc(OutDesc) && Sync.GetDesc(OutDesc);
	}

	void InitializeInterprocessObject(const FTextureShareCoreObjectDesc& InObjectDesc, const FTextureShareCoreSyncSettings& InSyncSettings)
	{
		Desc.Initialize(InObjectDesc);

		// Initialize sync settings first time
		Sync.Release();
		Sync.SyncSettings.UpdateInterprocessObjectSyncSettings(InObjectDesc, InSyncSettings);
		Sync.UpdateLastAccessTime();

		Data.Initialize();
	}

	void Release()
	{
		Desc.Release();
		Sync.Release();
		Data.Release();
	}

	void UpdateInterprocessObject(const FTextureShareCoreObjectDesc& InObjectDesc, const FTextureShareCoreSyncSettings& InSyncSettings)
	{
		Desc.Initialize(InObjectDesc);

		Sync.SyncSettings.UpdateInterprocessObjectSyncSettings(InObjectDesc, InSyncSettings);
		Sync.UpdateLastAccessTime();
	}
};
