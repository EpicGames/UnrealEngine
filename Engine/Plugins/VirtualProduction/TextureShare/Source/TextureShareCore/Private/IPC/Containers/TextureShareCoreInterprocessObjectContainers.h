// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Containers/TextureShareCoreContainers.h"
#include "IPC/Containers/TextureShareCoreInterprocessEnums.h"

/**
 * Data wrapper for serializer
 */
struct FTextureShareCoreObjectDataRef
	: public ITextureShareSerialize
{
	// Object ID
	FTextureShareCoreObjectDesc ObjectDesc;

	// Setup frame data on game thread before rendering call
	FTextureShareCoreData& Data;

public:
	virtual ~FTextureShareCoreObjectDataRef() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << ObjectDesc << Data;
	}

public:
	FTextureShareCoreObjectDataRef(const FTextureShareCoreObjectDesc& InObjectDesc, FTextureShareCoreData& InData)
		: ObjectDesc(InObjectDesc), Data(InData)
	{ }
};

/**
 * Data wrapper for serializer
 */
struct FTextureShareCoreObjectProxyDataRef
	: public ITextureShareSerialize
{
	// Object ID
	FTextureShareCoreObjectDesc ObjectDesc;

	// Setup frame data on game thread before rendering call
	FTextureShareCoreProxyData& ProxyData;

public:
	virtual ~FTextureShareCoreObjectProxyDataRef() = default;

	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream & Stream) override
	{
		return Stream << ObjectDesc << ProxyData;
	}

public:
	FTextureShareCoreObjectProxyDataRef(const FTextureShareCoreObjectDesc& InObjectDesc, FTextureShareCoreProxyData& InProxyData)
		: ObjectDesc(InObjectDesc), ProxyData(InProxyData)
	{ }
};
