// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPC/Containers/TextureShareCoreInterprocessObjectData.h"
#include "IPC/TextureShareCoreInterprocessSerializer.h"

/////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreInterprocessObjectDataHeader
/////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreInterprocessObjectDataHeader::Initialize()
{
	Release();

	LastWriteTime.Update();
}

void FTextureShareCoreInterprocessObjectDataHeader::Release()
{
	Type = ETextureShareCoreInterprocessObjectDataType::Undefined;
	Size = 0;
	LastWriteTime.Empty();
}

/////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreInterprocessObjectData
/////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreInterprocessObjectData::Write(FTextureShareCoreObjectDataRef& InObjectData)
{
	FTextureShareCoreSerializeStreamWrite(*this) << InObjectData;

	DataHeader.Type = ETextureShareCoreInterprocessObjectDataType::Frame;
	DataHeader.LastWriteTime.Update();

	return true;
}

bool FTextureShareCoreInterprocessObjectData::Write(FTextureShareCoreObjectProxyDataRef& InObjectProxyData)
{
	FTextureShareCoreSerializeStreamWrite(*this) << InObjectProxyData;

	DataHeader.Type = ETextureShareCoreInterprocessObjectDataType::FrameProxy;
	DataHeader.LastWriteTime.Update();

	return true;
}

bool FTextureShareCoreInterprocessObjectData::Read(FTextureShareCoreObjectData& OutObjectData) const
{
	if (DataHeader.Type == ETextureShareCoreInterprocessObjectDataType::Frame)
	{
		FTextureShareCoreSerializeStreamRead(*this) << OutObjectData;
		return true;
	}

	return false;
}

bool FTextureShareCoreInterprocessObjectData::Read(FTextureShareCoreObjectProxyData& OutObjectProxyData) const
{
	if (DataHeader.Type == ETextureShareCoreInterprocessObjectDataType::FrameProxy)
	{
		FTextureShareCoreSerializeStreamRead(*this) << OutObjectProxyData;
		return true;
	}

	return false;
}
