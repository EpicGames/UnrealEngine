// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IPC/Containers/TextureShareCoreInterprocessContainers.h"

/**
 * IPC object Data Header: define data size, type and access time
 * This structure fits the criteria of being POD ("Plain Old Data") for binary compatibility with direct memory access.
 */
struct FTextureShareCoreInterprocessObjectDataHeader
{
	// Type of serialized data in memory area
	ETextureShareCoreInterprocessObjectDataType Type;

	// Size of serialized data in memory area
	uint32 Size;

	// Time when serialized data was last written to memory
	FTextureShareCoreTimestump LastWriteTime;

public:
	void Initialize();
	void Release();
};

/**
 * IPC object Data Container: raw memory bytes for serializer
 * This structure fits the criteria of being POD ("Plain Old Data") for binary compatibility with direct memory access.
 */
struct FTextureShareCoreInterprocessObjectDataMemory
{
	// Maximum size of serialized data
	static auto constexpr MaxInterprocessObjectDataSize = (32 * 1024);

	// Memory area for serialized data
	uint8 Data[MaxInterprocessObjectDataSize];

public:
	// The 'Data' is references from the 'DataHeader', no cleanup required
	void Initialize()
	{ }

	void Release()
	{ }
};

/**
 * TS object memory container for serializer
 * This structure fits the criteria of being POD ("Plain Old Data") for binary compatibility with direct memory access.
 */
struct FTextureShareCoreInterprocessObjectData
{
	// Serialized data description
	FTextureShareCoreInterprocessObjectDataHeader DataHeader;

	// Serialized data container
	FTextureShareCoreInterprocessObjectDataMemory DataMemory;

public:
	bool IsEnabled(const ETextureShareCoreInterprocessObjectDataType InDataType) const
	{
		return DataHeader.Type == InDataType;
	}

	void Initialize()
	{
		DataHeader.Initialize();
		DataMemory.Initialize();
	}

	void Release()
	{
		DataHeader.Release();
		DataMemory.Release();
	}

public:
	// Serialize
	bool Write(struct FTextureShareCoreObjectDataRef& InObjectData);
	bool Write(struct FTextureShareCoreObjectProxyDataRef& InObjectProxyData);

	// De-Serialize
	bool Read(struct FTextureShareCoreObjectData& OutObjectData) const;
	bool Read(struct FTextureShareCoreObjectProxyData& OutObjectProxyData) const;
};
