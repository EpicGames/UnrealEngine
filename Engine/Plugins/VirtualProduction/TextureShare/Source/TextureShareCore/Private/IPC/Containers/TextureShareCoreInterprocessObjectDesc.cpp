// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPC/Containers/TextureShareCoreInterprocessObjectDesc.h"

/////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreInterprocessObjectDesc
/////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreInterprocessObjectDesc::Initialize(const FTextureShareCoreObjectDesc& InCoreObjectDesc)
{
	Release();

	ObjectGuid.Initialize(InCoreObjectDesc.ObjectGuid);
	ShareName.Initialize(InCoreObjectDesc.ShareName);

	ProcessName.Initialize(InCoreObjectDesc.ProcessDesc.ProcessId);
	ProcessGuid.Initialize(InCoreObjectDesc.ProcessDesc.ProcessGuid);

	ProcessType = InCoreObjectDesc.ProcessDesc.ProcessType;
}

void FTextureShareCoreInterprocessObjectDesc::Release()
{
	ObjectGuid.Empty();
	ShareName.Empty();

	ProcessName.Empty();
	ProcessGuid.Empty();

	ProcessType = ETextureShareProcessType::Undefined;
}

bool FTextureShareCoreInterprocessObjectDesc::GetDesc(FTextureShareCoreObjectDesc& OutDesc) const
{
	if (IsEnabled())
	{
		OutDesc.ProcessDesc.ProcessId = ProcessName.ToString();
		OutDesc.ProcessDesc.ProcessGuid = ProcessGuid.ToGuid();

		OutDesc.ProcessDesc.ProcessType = ProcessType;

		OutDesc.ShareName = ShareName.ToString();
		OutDesc.ObjectGuid = ObjectGuid.ToGuid();

		return true;
	}

	return false;
}
