// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IPC/Containers/TextureShareCoreInterprocessContainers.h"

/**
 * IPC object descriptor
 * This structure fits the criteria of being POD ("Plain Old Data") for binary compatibility with direct memory access.
 */
struct FTextureShareCoreInterprocessObjectDesc
{
	// Unique process guid. Generated unique for a local process
	FTextureShareCoreGuid ObjectGuid;

	// MD5 hash of the object name. Generated from a ShareName using the MD5 hash function.
	// Paired objects use identical hash values
	FTextureShareCoreStringHash ShareName;

	// MD5 hash of the process name. Generated from a ProcessName using the MD5 hash function.
	FTextureShareCoreStringHash ProcessName;

	// Unique process guid
	FTextureShareCoreGuid ProcessGuid;

	// The local process type
	ETextureShareProcessType ProcessType;

public:
	bool IsEnabled() const
	{
		return ObjectGuid.IsEmpty() == false;
	}

	bool Equals(const FTextureShareCoreGuid& InObjectGuid) const
	{
		return ObjectGuid.Equals(InObjectGuid);
	}

	bool Equals(const FTextureShareCoreObjectDesc& InObjectDesc) const
	{
		return ObjectGuid.Equals(FTextureShareCoreGuid::Create(InObjectDesc.ObjectGuid));
	}

	bool IsShareNameEquals(const FString& InShareName) const
	{
		return ShareName.Equals(FTextureShareCoreSMD5Hash::Create(InShareName));
	}

	bool IsShareNameEquals(const FTextureShareCoreStringHash& InShareNameHash) const
	{
		return ShareName.Equals(InShareNameHash);
	}

	bool IsShareNameEquals(const FTextureShareCoreSMD5Hash& InHash) const
	{
		return ShareName.Equals(InHash);
	}

	bool IsConnectable(const FTextureShareCoreInterprocessObjectDesc& In) const
	{
		return IsConnectable(In.ShareName.Hash, In.ProcessType);
	}

	bool IsConnectable(const FTextureShareCoreSMD5Hash& InHash, const ETextureShareProcessType InProcessType) const
	{
		if (!IsEnabled())
		{
			return false;
		}

		// The objects will be connect only if ShareName equal
		if (!IsShareNameEquals(InHash))
		{
			return false;
		}

		// Process connection rules by type:
		switch (ProcessType)
		{
		case ETextureShareProcessType::SDK:
			// Possible connections: [UE], [UE2UE]
			switch (InProcessType)
			{
			case ETextureShareProcessType::UE:
			case ETextureShareProcessType::UE2UE:
				break;
			default:
				return false;
			}
			break;

		case ETextureShareProcessType::UE:
			// Possible connections: [SDK]
			switch (InProcessType)
			{
			case ETextureShareProcessType::SDK:
				break;
			default:
				return false;
			}
			break;

		case ETextureShareProcessType::UE2UE:
			// Possible connections: [SDK], [UE2UE]
			switch (InProcessType)
			{
			case ETextureShareProcessType::SDK:
			case ETextureShareProcessType::UE2UE:
				break;
			default:
				return false;
			}
			break;

		case ETextureShareProcessType::Undefined:
			return false;

		default:
			break;
		}

		return true;
	}

public:
	void Initialize(const FTextureShareCoreObjectDesc& InCoreObjectDesc);
	void Release();
	bool GetDesc(FTextureShareCoreObjectDesc& OutDesc) const;
};
