// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPC/Containers/TextureShareCoreInterprocessObjectSync.h"

/////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreInterprocessObjectSync
/////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreInterprocessObjectSync::Release()
{
	SyncSettings.Release();
	SyncState.Release();

	LastAccessTime.Empty();

	bProcessStuck = false;
}

bool FTextureShareCoreInterprocessObjectSync::GetDesc(FTextureShareCoreObjectDesc& OutDesc) const
{
	if (IsEnabled())
	{
		// Return access time in cycle64
		OutDesc.Sync.LastAccessTime = LastAccessTime.Time;

		// Sync logic rules
		OutDesc.Sync.SyncStepSettings = SyncSettings.GetData();

		// and current sync state
		SyncState.Read(OutDesc.Sync.SyncState);

		return true;
	}

	return false;
}
