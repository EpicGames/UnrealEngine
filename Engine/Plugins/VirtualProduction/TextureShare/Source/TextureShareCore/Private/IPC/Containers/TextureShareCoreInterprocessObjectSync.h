// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "IPC/Containers/TextureShareCoreInterprocessContainers.h"
#include "IPC/Containers/TextureShareCoreInterprocessObjectSyncSettings.h"
#include "IPC/Containers/TextureShareCoreInterprocessObjectSyncState.h"

/**
 * IPC object data: Sync
  * This structure fits the criteria of being POD ("Plain Old Data") for binary compatibility with direct memory access.
 */
struct FTextureShareCoreInterprocessObjectSync
{
	// interprocess sync logic
	FTextureShareCoreInterprocessObjectSyncState SyncState;

	// Sync settings
	FTextureShareCoreInterprocessObjectSyncSettings SyncSettings;

	// Last access time from this object
	FTextureShareCoreTimestump LastAccessTime;

	// This flag set after any frame begin timeout\connect for all unused processes
	bool bProcessStuck;

public:
	void Release();
	bool GetDesc(FTextureShareCoreObjectDesc& OutDesc) const;

public:
	////////////////////////////////////////////////////////////////////////////////////////////
	// States
	////////////////////////////////////////////////////////////////////////////////////////////

	// Touch the time avery time when process get own data
	void UpdateLastAccessTime()
	{
		LastAccessTime.Update();

		// when process is alive, reset
		bProcessStuck = false;
	}

	bool IsEnabled() const
	{
		if (LastAccessTime.IsEmpty() || bProcessStuck || SyncSettings.IsProcessLost(LastAccessTime.GetElapsedMilisecond()))
		{
			return false;
		}

		return true;
	}

	////////////////////////////////////////////////////////////////////////////////////////////
	// Sync support
	////////////////////////////////////////////////////////////////////////////////////////////

	ETextureShareInterprocessObjectSyncBarrierState GetBarrierState(const FTextureShareCoreInterprocessObjectSync& InObjectSync) const
	{
		const ETextureShareSyncStep  InStep  = InObjectSync.SyncState.Step;
		const ETextureShareSyncState InState = InObjectSync.SyncState.State;

		switch (SyncState.Step)
		{
		case ETextureShareSyncStep::Undefined:
		case ETextureShareSyncStep::InterprocessConnection:
			// Synchronization is invalid for this process
			break;

		default:
			// When the synchronization is valid for the current process, the following race rules are used:
			if (!SyncSettings.IsStepEnabled(InStep))
			{
				// Skip steps unused by this process
				return ETextureShareInterprocessObjectSyncBarrierState::UnusedSyncStep;
			}

			if (!InObjectSync.SyncSettings.IsStepEnabled(SyncState.Step))
			{
				// Targer process dont know about current step
				return ETextureShareInterprocessObjectSyncBarrierState::Wait;
			}
		}

		return SyncState.GetBarrierState(InObjectSync.SyncState);
	}

	void MarkProcessStuck()
	{
		// Other process failed to connect with this process
		// mark as potencially stuck
		bProcessStuck = true;
	}

	bool BeginSyncBarrier(const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass, const ETextureShareSyncStep InNextSyncStep)
	{
		UpdateLastAccessTime();
		return SyncState.BeginSyncBarrier(InSyncStep, InSyncPass, InNextSyncStep);
	}

	bool AcceptSyncBarrier(const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass)
	{
		UpdateLastAccessTime();
		return SyncState.AcceptSyncBarrier(InSyncStep, InSyncPass);
	}

	bool IsBarrierCompleted(const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass) const
	{
		if (SyncState.Step == InSyncStep)
		{
			switch (InSyncPass)
			{
			case ETextureShareSyncPass::Enter:
				return SyncState.State == ETextureShareSyncState::EnterCompleted;

			case ETextureShareSyncPass::Exit:
				return SyncState.State == ETextureShareSyncState::ExitCompleted;

			default:
				break;
			}
		}
		
		return false;
	}

	void ResetSync()
	{
		UpdateLastAccessTime();
		SyncState.ResetSync();
	}

	const FTextureShareCoreInterprocessObjectSyncState& GetSyncState() const
	{
		return SyncState;
	}

	const FTextureShareCoreInterprocessObjectSyncSettings& GetSyncSettings() const
	{
		return SyncSettings;
	}
};
