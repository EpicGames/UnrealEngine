// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPC/Containers/TextureShareCoreInterprocessObjectSyncSettings.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreInterprocessObjectSyncSettings
//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreInterprocessObjectSyncSettings::UpdateInterprocessObjectSyncSettings(const FTextureShareCoreObjectDesc& InObjectDesc, const FTextureShareCoreSyncSettings& InSyncSettings)
{
	// Update process name rules
	AllowedProcessNames.Initialize(InSyncSettings.FrameConnectionSettings.AllowedProcessNames);
	BannedProcessNames.Initialize(InSyncSettings.FrameConnectionSettings.BannedProcessNames);

	// Update dead timeout value for local process
	ProcessLostStatusTimeOut = InSyncSettings.TimeoutSettings.ProcessLostStatusTimeOut;

	// Update sync settings for object desc
	Data = InObjectDesc.Sync.SyncStepSettings;
}

void FTextureShareCoreInterprocessObjectSyncSettings::Release()
{
	FPlatformMemory::Memset(&Data, 0, sizeof(Data));

	ProcessLostStatusTimeOut = 0;
}
