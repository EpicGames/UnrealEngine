// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Containers/TextureShareCoreEnums.h"
#include "IPC/Containers/TextureShareCoreInterprocessContainers.h"

/**
 * IPC object data: SyncSettings
 * This structure fits the criteria of being POD ("Plain Old Data") for binary compatibility with direct memory access.
 */
struct FTextureShareCoreInterprocessObjectSyncSettings
{
	// Synchronization settings for allowed/forbidden process names
	FTextureShareCoreSMD5HashList AllowedProcessNames;
	FTextureShareCoreSMD5HashList BannedProcessNames;

	// Bit-storage for sync steps used by this process
	// 64 bits. Max steps=64
	uint64 Data;

	// process status loss timeout for local process
	int32 ProcessLostStatusTimeOut;

public:
	void UpdateInterprocessObjectSyncSettings(const struct FTextureShareCoreObjectDesc& InObjectDesc, const struct FTextureShareCoreSyncSettings& InSyncSettings);
	void Release();

public:
	bool IsStepEnabled(const ETextureShareSyncStep InStep) const
	{
		if (InStep == ETextureShareSyncStep::InterprocessConnection)
		{
			return true;
		}

		const uint8 BitIndex = (uint8)InStep;
		
		if (BitIndex >= 0 && BitIndex < 64)
		{
			return (Data & (1ULL << BitIndex)) != 0;
		}

		return false;
	}

	bool IsProcessLost(const uint32 InElapsedMilisecond) const
	{
		// now object is dead if last access time expired.
		// If the process is alive, it is re-created on-demant and re-connected
		// '-1' - disable this option
		if (ProcessLostStatusTimeOut)
		{
			if (InElapsedMilisecond > static_cast<uint32>(ProcessLostStatusTimeOut))
			{
				return true;
			}
		}

		return false;
	}

	// The rules accepted in both directions
	bool IsShouldBeConnected(const FTextureShareCoreStringHash& LocalProcessName, const FTextureShareCoreStringHash& RemoteProcessName, const FTextureShareCoreInterprocessObjectSyncSettings& RemoteProcessSyncSettings) const
	{
		return RemoteProcessSyncSettings.IsProcessNameShouldBeConnected(LocalProcessName.Hash)
			&& IsProcessNameShouldBeConnected(RemoteProcessName.Hash);
	}

	uint64 GetData() const
	{
		return Data;
	}

protected:
	bool IsProcessNameShouldBeConnected(const FTextureShareCoreSMD5Hash& InProcessName) const
	{
		const bool bUseAllowList = AllowedProcessNames.IsEmpty() == false;
		const bool bUseBanList = BannedProcessNames.IsEmpty() == false;

		if (bUseAllowList || bUseBanList)
		{
			const bool bAllowListResult = AllowedProcessNames.Find(InProcessName) != INDEX_NONE;
			const bool bBanListResult = BannedProcessNames.Find(InProcessName) != INDEX_NONE;

			if (bUseAllowList && !bAllowListResult)
			{
				return false;
			}

			if (bUseBanList && bBanListResult)
			{
				return false;
			}
		}

		return true;
	}
};
