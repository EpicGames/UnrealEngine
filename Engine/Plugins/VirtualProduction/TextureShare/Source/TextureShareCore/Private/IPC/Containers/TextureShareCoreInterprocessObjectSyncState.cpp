// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPC/Containers/TextureShareCoreInterprocessObjectSyncState.h"
#include "IPC/Containers/TextureShareCoreInterprocessObjectSync.h"

#include "IPC/TextureShareCoreInterprocessHelpers.h"
#include "Module/TextureShareCoreLog.h"

using namespace UE::TextureShareCore;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreInterprocessObjectSyncState
//////////////////////////////////////////////////////////////////////////////////////////////
ETextureShareInterprocessObjectSyncBarrierState FTextureShareCoreInterprocessObjectSyncState::HandleBarrierStateResult(const FTextureShareCoreInterprocessObjectSyncState& InSync, const ETextureShareInterprocessObjectSyncBarrierState InBarrierState) const
{
	switch (InBarrierState)
	{
	case ETextureShareInterprocessObjectSyncBarrierState::Accept:
	case ETextureShareInterprocessObjectSyncBarrierState::AcceptConnection:
	case ETextureShareInterprocessObjectSyncBarrierState::Wait:
	case ETextureShareInterprocessObjectSyncBarrierState::WaitConnection:
		break;

	default:
		// Add some log
		UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("\t\tBarrierState=%s : %s ask from %s"), GetTEXT(InBarrierState), *ToString(InSync), *ToString(*this));
		break;
	}

	return InBarrierState;
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreInterprocessObjectSyncState::HandleLogicBroken()
{
	// Logic broken
	Release();

	return false;
}

ETextureShareInterprocessObjectSyncBarrierState FTextureShareCoreInterprocessObjectSyncState::FTextureShareCoreInterprocessObjectSyncState::GetConnectionBarrierState_Enter(const FTextureShareCoreInterprocessObjectSyncState& InSync) const
{
	// InSync.Step == ETextureShareSyncStep::InterprocessConnection
	// InSync.State == ETextureShareSyncState::Enter
	switch (State)
	{
	case ETextureShareSyncState::ExitCompleted:
	case ETextureShareSyncState::Completed:
		if (InSync.Step != Step)
		{
			return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::WaitConnection);
		}
		break;

	case ETextureShareSyncState::Enter:
	case ETextureShareSyncState::EnterCompleted:
		if (InSync.Step == Step)
		{
			return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::AcceptConnection);
		}
		break;

	default:
		break;
	}

	return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::FrameSyncLost);
}

ETextureShareInterprocessObjectSyncBarrierState FTextureShareCoreInterprocessObjectSyncState::GetConnectionBarrierState_EnterCompleted(const FTextureShareCoreInterprocessObjectSyncState& InSync) const
{
	// InSync.Step == ETextureShareSyncStep::InterprocessConnection
	// InSync.State == ETextureShareSyncState::EnterCompleted
	switch (State)
	{
	case ETextureShareSyncState::Enter:
		if (InSync.Step == Step)
		{
			return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::WaitConnection);
		}
		break;

	case ETextureShareSyncState::EnterCompleted:
	case ETextureShareSyncState::Exit:
		if (InSync.Step == Step)
		{
			return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::AcceptConnection);
		}
		break;

	default:
		break;
	}

	return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::FrameSyncLost);
}

ETextureShareInterprocessObjectSyncBarrierState FTextureShareCoreInterprocessObjectSyncState::GetConnectionBarrierState(const FTextureShareCoreInterprocessObjectSyncState& InSync) const
{
	check(InSync.Step == ETextureShareSyncStep::InterprocessConnection);

	switch (InSync.State)
	{
	case ETextureShareSyncState::Enter:          return GetConnectionBarrierState_Enter(InSync);
	case ETextureShareSyncState::EnterCompleted: return GetConnectionBarrierState_EnterCompleted(InSync);
	default:
		break;
	}

	return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::InvalidLogic);
}

ETextureShareInterprocessObjectSyncBarrierState FTextureShareCoreInterprocessObjectSyncState::GetBarrierState_Enter(const FTextureShareCoreInterprocessObjectSyncState& InSync) const
{
	// InSync.State == ETextureShareSyncState::Enter
	switch (State)
	{
	case ETextureShareSyncState::ExitCompleted:
	case ETextureShareSyncState::Completed:
		if (InSync.Step != Step)
		{
			return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::Wait);
		}
		break;

	case ETextureShareSyncState::Enter:
	case ETextureShareSyncState::EnterCompleted:
		if (InSync.Step == Step)
		{
			return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::Accept);
		}
		break;

	default:
		break;
	}

	return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::FrameSyncLost);
}

ETextureShareInterprocessObjectSyncBarrierState FTextureShareCoreInterprocessObjectSyncState::GetBarrierState_EnterCompleted(const FTextureShareCoreInterprocessObjectSyncState& InSync) const
{
	// InSync.State == ETextureShareSyncState::EnterCompleted
	if (InSync.Step > Step)
	{
		// logic overrun
		return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::Accept);
	}

	switch (State)
	{
	case ETextureShareSyncState::Enter:
		if (InSync.Step == Step)
		{
			return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::Wait);
		}
		break;

	case ETextureShareSyncState::EnterCompleted:
	case ETextureShareSyncState::Exit:
		if (InSync.Step == Step)
		{
			return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::Accept);
		}
		break;

	default:
		break;
	}

	return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::FrameSyncLost);
}

ETextureShareInterprocessObjectSyncBarrierState FTextureShareCoreInterprocessObjectSyncState::GetBarrierState_Exit(const FTextureShareCoreInterprocessObjectSyncState& InSync) const
{
	// InSync.State == ETextureShareSyncState::Exit
	switch (State)
	{
	case ETextureShareSyncState::Enter:
	case ETextureShareSyncState::EnterCompleted:
		if (InSync.Step == Step)
		{
			return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::Wait);
		}
		break;

	case ETextureShareSyncState::Exit:
	case ETextureShareSyncState::ExitCompleted:
		if (InSync.Step == Step)
		{
			return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::Accept);
		}
		break;

	default:
		break;
	}

	return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::FrameSyncLost);
}

ETextureShareInterprocessObjectSyncBarrierState FTextureShareCoreInterprocessObjectSyncState::GetBarrierState_ExitCompleted(const FTextureShareCoreInterprocessObjectSyncState& InSync) const
{
	// InSync.State = ETextureShareSyncState::ExitCompleted
	switch (State)
	{
	case ETextureShareSyncState::Enter:
		if (Step > InSync.Step)
		{
			// logic overrun
			return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::Accept);
		}
		break;

	case ETextureShareSyncState::EnterCompleted:
	case ETextureShareSyncState::Exit:
		if (InSync.Step == Step)
		{
			return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::Wait);
		}
		break;

	case ETextureShareSyncState::ExitCompleted:
	case ETextureShareSyncState::Completed:
		if (InSync.Step == Step)
		{
			return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::Accept);
		}
		break;

	default:
		break;
	}

	return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::FrameSyncLost);
}

ETextureShareInterprocessObjectSyncBarrierState FTextureShareCoreInterprocessObjectSyncState::GetBarrierState(const FTextureShareCoreInterprocessObjectSyncState& InSync) const
{
	if (Step == ETextureShareSyncStep::Undefined)
	{
		return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::WaitConnection);
	}

	if (State == ETextureShareSyncState::Undefined)
	{
		return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::InvalidLogic);
	}

	if (InSync.Step == ETextureShareSyncStep::InterprocessConnection)
	{
		switch (InSync.State)
		{
		case ETextureShareSyncState::Enter:
		case ETextureShareSyncState::EnterCompleted:
			return GetConnectionBarrierState(InSync);

		default:
			break;
		}
		
	}

	switch (InSync.State)
	{
	case ETextureShareSyncState::Enter:          return GetBarrierState_Enter(InSync);
	case ETextureShareSyncState::EnterCompleted: return GetBarrierState_EnterCompleted(InSync);
	case ETextureShareSyncState::Exit:           return GetBarrierState_Exit(InSync);
	case ETextureShareSyncState::ExitCompleted:  return GetBarrierState_ExitCompleted(InSync);
	default:
		break;
	}

	return HandleBarrierStateResult(InSync, ETextureShareInterprocessObjectSyncBarrierState::InvalidLogic);
}

bool FTextureShareCoreInterprocessObjectSyncState::BeginSyncBarrier(const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass, const ETextureShareSyncStep InNextSyncStep)
{
	switch (State)
	{
	case ETextureShareSyncState::Undefined:
		// support only for new frame connection
		if (InSyncStep != ETextureShareSyncStep::InterprocessConnection || InSyncPass != ETextureShareSyncPass::Enter)
		{
			return HandleLogicBroken();
		}

		// Enter sync conection
		State = ETextureShareSyncState::Enter;
		break;

	case ETextureShareSyncState::Completed:
		// Enter into a new syn barrier only from the 'Completed' state
		if (InSyncPass != ETextureShareSyncPass::Enter)
		{
			return HandleLogicBroken();
		}

		// Enter new sync step
		State = ETextureShareSyncState::Enter;
		break;

	case ETextureShareSyncState::EnterCompleted:
		if (InSyncPass != ETextureShareSyncPass::Exit || Step != InSyncStep)
		{
			return HandleLogicBroken();
		}

		// Enter to step 'Exit' sync step
		State = ETextureShareSyncState::Exit;
		break;

	default:
		return HandleLogicBroken();
	};

	if (InSyncStep != Step || NextStep != InNextSyncStep)
	{
		PrevStep = Step;
		Step = InSyncStep;
		NextStep = InNextSyncStep;
	}

	return true;
}

bool FTextureShareCoreInterprocessObjectSyncState::AcceptSyncBarrier(const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass)
{
	if (InSyncStep == Step)
	{
		switch (InSyncPass)
		{
		case ETextureShareSyncPass::Enter:
			if (State == ETextureShareSyncState::Enter)
			{
				State = ETextureShareSyncState::EnterCompleted;

				return true;
			}
			break;

		case ETextureShareSyncPass::Exit:
			if (State == ETextureShareSyncState::Exit)
			{
				State = ETextureShareSyncState::ExitCompleted;

				return true;
			}
			break;

		case ETextureShareSyncPass::Complete:
			if (State == ETextureShareSyncState::ExitCompleted)
			{
				State = ETextureShareSyncState::Completed;

				return true;
			}
			break;

		default:
			break;
		}
	}

	return HandleLogicBroken();
}

//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreInterprocessObjectSyncState::Read(FTextureShareCoreObjectSyncState& OutSyncState) const
{
	OutSyncState.State = State;
	OutSyncState.Step = Step;
	OutSyncState.NextStep = NextStep;
	OutSyncState.PrevStep = PrevStep;
}

void FTextureShareCoreInterprocessObjectSyncState::ResetSync()
{
	Release();
}

//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreInterprocessObjectSyncState::Initialize()
{
	Release();
}

void FTextureShareCoreInterprocessObjectSyncState::Release()
{
	Step = ETextureShareSyncStep::Undefined;
	State = ETextureShareSyncState::Undefined;

	NextStep = ETextureShareSyncStep::Undefined;
	PrevStep = ETextureShareSyncStep::Undefined;
}
