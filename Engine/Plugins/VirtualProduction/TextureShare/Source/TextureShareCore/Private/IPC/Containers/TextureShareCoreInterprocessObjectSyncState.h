// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IPC/Containers/TextureShareCoreInterprocessEnums.h"
#include "Containers/TextureShareCoreContainers.h"

/**
 * IPC object data: SyncState
 * This structure fits the criteria of being POD ("Plain Old Data") for binary compatibility with direct memory access.
 */
struct FTextureShareCoreInterprocessObjectSyncState
{
	// Barrier sync step
	ETextureShareSyncStep  Step;

	// Barrier sync state
	ETextureShareSyncState State;

	// Next sync step
	ETextureShareSyncStep NextStep;

	// Prev sync step
	ETextureShareSyncStep PrevStep;

public:
	// Enter barrier
	bool BeginSyncBarrier(const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass, const ETextureShareSyncStep InNextSyncStep);

	// Accept barrier (Called agter BeginSyncBarrier)
	bool AcceptSyncBarrier(const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass);

	// Whe error or sync is lost
	void ResetSync();

	// Return barrier state
	ETextureShareInterprocessObjectSyncBarrierState GetBarrierState(const FTextureShareCoreInterprocessObjectSyncState& InSync) const;

	// Export barrier state from compressed POD to UE struct
	void Read(FTextureShareCoreObjectSyncState& OutSyncState) const;

public:
	void Initialize();
	void Release();

private:
	bool HandleLogicBroken();

	ETextureShareInterprocessObjectSyncBarrierState GetConnectionBarrierState(const FTextureShareCoreInterprocessObjectSyncState& InSync) const;
	ETextureShareInterprocessObjectSyncBarrierState GetConnectionBarrierState_Enter(const FTextureShareCoreInterprocessObjectSyncState& InSync) const;
	ETextureShareInterprocessObjectSyncBarrierState GetConnectionBarrierState_EnterCompleted(const FTextureShareCoreInterprocessObjectSyncState& InSync) const;

	ETextureShareInterprocessObjectSyncBarrierState GetBarrierState_Enter(const FTextureShareCoreInterprocessObjectSyncState& InSync) const;
	ETextureShareInterprocessObjectSyncBarrierState GetBarrierState_EnterCompleted(const FTextureShareCoreInterprocessObjectSyncState& InSync) const;
	ETextureShareInterprocessObjectSyncBarrierState GetBarrierState_Exit(const FTextureShareCoreInterprocessObjectSyncState& InSync) const;
	ETextureShareInterprocessObjectSyncBarrierState GetBarrierState_ExitCompleted(const FTextureShareCoreInterprocessObjectSyncState& InSync) const;

	ETextureShareInterprocessObjectSyncBarrierState HandleBarrierStateResult(const FTextureShareCoreInterprocessObjectSyncState& InSync, const ETextureShareInterprocessObjectSyncBarrierState InBarrierState) const;
};
