// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPC/TextureShareCoreInterprocessEvent.h"

#include "ProfilingDebugging/CsvProfiler.h"
#include "Stats/Stats.h"
#include "Stats/ThreadIdleStats.h"

#include "Module/TextureShareCoreLog.h"

// Platform Windows
#include "Windows/AllowWindowsPlatformTypes.h"

namespace UE::TextureShareCore::InterprocessEvent
{
	static TSharedPtr<FEvent, ESPMode::ThreadSafe> CreateNamedInterprocessEvent(const FString& EventName, const bool bOpenExistEvent, const void* InSecurityAttributes)
	{
		const FString GlobalEventName = FString::Printf(TEXT("Global\\%s"), *EventName);

		HANDLE EventHandle = NULL;

		if (bOpenExistEvent)
		{
			const DWORD AccessRights = EVENT_ALL_ACCESS | EVENT_MODIFY_STATE;//SYNCHRONIZE | EVENT_MODIFY_STATE;
			EventHandle = OpenEventA(AccessRights, false, TCHAR_TO_ANSI(*GlobalEventName));
			if (NULL == EventHandle)
			{
				UE_LOGF(LogTextureShareCore, Warning, "OpenEvent(AccessRights=0x%08x, bInherit=false, Name='%ls') failed with LastError = %d", AccessRights, *EventName, GetLastError());

				return nullptr;
			}
		}
		else
		{
			// Create new event
			EventHandle = CreateEventA((SECURITY_ATTRIBUTES*)InSecurityAttributes, true, false, TCHAR_TO_ANSI(*GlobalEventName));
			if (NULL == EventHandle)
			{
				UE_LOGF(LogTextureShareCore, Warning, "CreateEvent(NULL, bInherit=false, Name='%ls') failed with LastError = %d", *EventName, GetLastError());

				return nullptr;
			}
		}

		return MakeShared<FTextureShareCoreInterprocessEventWin, ESPMode::ThreadSafe>(EventHandle);
	}

	static DWORD WaitForSingleInterprocessEvent(const HANDLE EventHandle, DWORD dwMilliseconds)
	{
		return WaitForSingleObject(EventHandle, dwMilliseconds);
	}

	static bool SetInterprocessEvent(const HANDLE EventHandle)
	{
		return SetEvent(EventHandle);
	}

	static bool ResetInterprocessEvent(const HANDLE EventHandle)
	{
		return ResetEvent(EventHandle);
	}

	static bool CloseInterprocessEvent(const HANDLE EventHandle)
	{
		return CloseHandle(EventHandle);
	}
};
using namespace UE::TextureShareCore::InterprocessEvent;

#include "Windows/HideWindowsPlatformTypes.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreInterprocessEventWin
//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreInterprocessEventWin::ReleaseEvent()
{
	if (Event != nullptr)
	{
		CloseInterprocessEvent(Event);
	}
}

void FTextureShareCoreInterprocessEventWin::Trigger()
{
	TriggerForStats();

	check(Event);
	SetInterprocessEvent(Event);
}

void FTextureShareCoreInterprocessEventWin::Reset()
{
	ResetForStats();

	check(Event);
	ResetInterprocessEvent(Event);
}

bool FTextureShareCoreInterprocessEventWin::Wait(uint32 MaxMillisecondsToWait, const bool bIgnoreThreadIdleStats)
{
	check(Event);

	WaitForStats();

#if !TEXTURESHARECORE_SDK
	CSV_SCOPED_WAIT(MaxMillisecondsToWait);
	UE::Stats::FThreadIdleStats::FScopeIdle Scope(bIgnoreThreadIdleStats);
#endif

	if (!MaxMillisecondsToWait)
	{
		if (WaitForSingleInterprocessEvent(Event, INFINITE) == WAIT_OBJECT_0)
		{
			return true;
		}
	}
	else
	{
		if (WaitForSingleInterprocessEvent(Event, MaxMillisecondsToWait) == WAIT_OBJECT_0)
		{
			return true;
		}
	}

	return false;
}

//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<FEvent, ESPMode::ThreadSafe> FTextureShareCoreInterprocessEventWin::CreateInterprocessEvent(const FGuid& InEventGuid, const void* InSecurityAttributes)
{
	FString EventName = InEventGuid.ToString(EGuidFormats::DigitsWithHyphensInBraces);
	return CreateNamedInterprocessEvent(EventName, false, InSecurityAttributes);
}

TSharedPtr<FEvent, ESPMode::ThreadSafe> FTextureShareCoreInterprocessEventWin::OpenInterprocessEvent(const FGuid& InEventGuid, const void* InSecurityAttributes)
{
	FString EventName = InEventGuid.ToString(EGuidFormats::DigitsWithHyphensInBraces);
	return CreateNamedInterprocessEvent(EventName, true, InSecurityAttributes);
}
