// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "HAL/Event.h"

/**
 * IPC events (Windows)
 */
class FTextureShareCoreInterprocessEventWin
	: public FEvent
{
public:
	FTextureShareCoreInterprocessEventWin(Windows::HANDLE InEvent)
		: Event(InEvent)
	{ }

	virtual ~FTextureShareCoreInterprocessEventWin()
	{
		ReleaseEvent();
	}

	// FEvent
	virtual bool Create(bool bIsManualReset = false) override
	{
		return false;
	}

	virtual void Trigger() override;
	virtual void Reset() override;
	virtual bool Wait(uint32 MaxMillisecondsToWait, const bool bIgnoreThreadIdleStats = false) override;
	virtual bool IsManualReset() override
	{
		return true;
	}
	//~ FEvent

	Windows::HANDLE GetEventHandle() const
	{
		return Event;
	}

protected:
	void ReleaseEvent();

public:
	static TSharedPtr<FEvent, ESPMode::ThreadSafe> CreateInterprocessEvent(const FGuid& InEventGuid, const void* InSecurityAttributes);
	static TSharedPtr<FEvent, ESPMode::ThreadSafe> OpenInterprocessEvent(const FGuid& InEventGuid, const void* InSecurityAttributes);

private:
	/** Holds the handle to the event. */
	Windows::HANDLE Event;
};
