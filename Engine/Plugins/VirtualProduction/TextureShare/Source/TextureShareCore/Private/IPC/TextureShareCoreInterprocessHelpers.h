// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Core/TextureShareCoreHelpers.h"

/**
 * Enums to text wrappers for logging
 */
namespace UE::TextureShareCore
{
	static constexpr auto GetTEXT(const ETextureShareInterprocessObjectSyncBarrierState InBarrierState)
	{
		switch (InBarrierState)
		{
		case ETextureShareInterprocessObjectSyncBarrierState::Undefined: return TEXT("Undefined");
		case ETextureShareInterprocessObjectSyncBarrierState::UnusedSyncStep: return TEXT("UnusedSyncStep");

		case ETextureShareInterprocessObjectSyncBarrierState::WaitConnection: return TEXT("WaitConnection");
		case ETextureShareInterprocessObjectSyncBarrierState::AcceptConnection: return TEXT("AcceptConnection");

		case ETextureShareInterprocessObjectSyncBarrierState::Wait: return TEXT("Wait");
		case ETextureShareInterprocessObjectSyncBarrierState::Accept: return TEXT("Accept");

		case ETextureShareInterprocessObjectSyncBarrierState::ObjectLost: return TEXT("ObjectLost");
		case ETextureShareInterprocessObjectSyncBarrierState::FrameSyncLost: return TEXT("FrameSyncLost");

		case ETextureShareInterprocessObjectSyncBarrierState::InvalidLogic: return TEXT("InvalidLogic");
		}

		return TEXT("Unknown");
	};

	static constexpr auto GetTEXT(const ETextureShareCoreInterprocessObjectFrameSyncState InSyncState)
	{
		switch (InSyncState)
		{
		case  ETextureShareCoreInterprocessObjectFrameSyncState::Undefined: return TEXT("Undefined");
		case  ETextureShareCoreInterprocessObjectFrameSyncState::FrameSyncLost: return TEXT("FrameSyncLost");

		case  ETextureShareCoreInterprocessObjectFrameSyncState::NewFrame: return TEXT("NewFrame");
		case  ETextureShareCoreInterprocessObjectFrameSyncState::FrameConnected: return TEXT("FrameConnected");
		case  ETextureShareCoreInterprocessObjectFrameSyncState::FrameBegin: return TEXT("FrameBegin");
		case  ETextureShareCoreInterprocessObjectFrameSyncState::FrameEnd: return TEXT("FrameEnd");

		case  ETextureShareCoreInterprocessObjectFrameSyncState::FrameProxyBegin:  return TEXT("FrameProxyBegin");
		case  ETextureShareCoreInterprocessObjectFrameSyncState::FrameProxyEnd:  return TEXT("FrameProxyEnd");
		default:
			break;
		}

		return TEXT("Unknown");
	};

	static FString ToString(const FTextureShareCoreInterprocessObjectSyncState& InObjectSyncState)
	{
		return FString::Printf(TEXT("%s[%s]{%s < %s}"), GetTEXT(InObjectSyncState.Step), GetTEXT(InObjectSyncState.State), GetTEXT(InObjectSyncState.NextStep), GetTEXT(InObjectSyncState.PrevStep));
	}

	static FString ToString(const FTextureShareCoreInterprocessObject& InObject)
	{
		return FString::Printf(TEXT("%s(%s)"), *InObject.Desc.ProcessName.ToString(), *ToString(InObject.Sync.GetSyncState()));
	}

	static FString ToString(const TArray<FTextureShareCoreInterprocessObject*>& InObjects)
	{
		FString Result;
		for (const FTextureShareCoreInterprocessObject* It : InObjects)
		{
			if (!Result.IsEmpty())
			{
				Result += TEXT(", ");
			}

			Result += ToString(*It);
		}

		return Result;
	}
};
