// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPC/TextureShareCoreInterprocessMemoryRegion.h"
#include "IPC/Containers/TextureShareCoreInterprocessMemory.h"

#include "Module/TextureShareCoreLog.h"

#include "Windows/WindowsPlatformProcess.h"
#include "Logging/LogScopedVerbosityOverride.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreInterprocessMemoryRegion
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareCoreInterprocessMemoryRegion::FTextureShareCoreInterprocessMemoryRegion(const FString& InSharedMemoryRegionName)
	: SharedMemoryRegionName(InSharedMemoryRegionName)
{ }

FTextureShareCoreInterprocessMemoryRegion::~FTextureShareCoreInterprocessMemoryRegion()
{
	ReleaseInterprocessObjects();
}

bool FTextureShareCoreInterprocessMemoryRegion::Initialize()
{
	if (PlatformMemoryRegion)
	{
		return false;
	}

	uint32 AccessMode = FPlatformMemory::ESharedMemoryAccess::Read | FPlatformMemory::ESharedMemoryAccess::Write;
	SIZE_T SharedMemorySize = sizeof(FTextureShareCoreInterprocessMemory);

	LOG_SCOPE_VERBOSITY_OVERRIDE(LogHAL, ELogVerbosity::Error);

	// Open existing shared memory region:
	FPlatformMemory::FSharedMemoryRegion* SharedMemoryRegion = FPlatformMemory::MapNamedSharedMemoryRegion(*SharedMemoryRegionName, false, AccessMode, SharedMemorySize);
	if (SharedMemoryRegion)
	{
		UE_TS_LOG(LogTextureShareCore, Log, TEXT("Open exist shared memory region '%s'."), *SharedMemoryRegionName);

		// Use exist shared memory
		PlatformMemoryRegion = SharedMemoryRegion;

		return true;
	}
	else
	{
		// Open new shared memory region
		SharedMemoryRegion = FPlatformMemory::MapNamedSharedMemoryRegion(*SharedMemoryRegionName, true, AccessMode, SharedMemorySize);
		if (SharedMemoryRegion)
		{
			UE_TS_LOG(LogTextureShareCore, Log, TEXT("Create new shared memory region '%s'. Memory cleared."), *SharedMemoryRegionName);

			// initialize memory with zeroes
			check(SharedMemoryRegion->GetAddress());
			FMemory::Memzero(SharedMemoryRegion->GetAddress(), SharedMemoryRegion->GetSize());

			PlatformMemoryRegion = SharedMemoryRegion;

			return true;
		}
	}

	UE_TS_LOG(LogTextureShareCore, Error, TEXT("Failed to open shared memory area '%s'"), *SharedMemoryRegionName);

	return false;
}

FTextureShareCoreInterprocessMemory* FTextureShareCoreInterprocessMemoryRegion::GetInterprocessMemory() const
{
	if (FPlatformMemory::FSharedMemoryRegion* SharedMemoryRegion = static_cast<FPlatformMemory::FSharedMemoryRegion*>(PlatformMemoryRegion))
	{
		return static_cast<FTextureShareCoreInterprocessMemory*>(SharedMemoryRegion->GetAddress());
	}

	return nullptr;
}

bool FTextureShareCoreInterprocessMemoryRegion::CleanupInterprocessObjects() const
{
	// NOT IMPLEMENTED
	return false;
}

void FTextureShareCoreInterprocessMemoryRegion::ReleaseInterprocessObjects()
{
	if (FPlatformMemory::FSharedMemoryRegion* SharedMemoryRegion = static_cast<FPlatformMemory::FSharedMemoryRegion*>(PlatformMemoryRegion))
	{
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogHAL, ELogVerbosity::Error);

		UE_TS_LOG(LogTextureShareCore, Log, TEXT("Release shared memory area '%s'"), *SharedMemoryRegionName);

		FPlatformMemory::UnmapNamedSharedMemoryRegion(SharedMemoryRegion);
		SharedMemoryRegion = nullptr;
	}
}
