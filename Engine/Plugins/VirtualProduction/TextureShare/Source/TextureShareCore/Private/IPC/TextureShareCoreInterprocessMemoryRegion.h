// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

/**
 * IPC Shared Memory
 */
class FTextureShareCoreInterprocessMemoryRegion
{
public:
	FTextureShareCoreInterprocessMemoryRegion(const FString& InSharedMemoryRegionName);
	~FTextureShareCoreInterprocessMemoryRegion();

	// Initialize shared memory resources
	bool Initialize();

	struct FTextureShareCoreInterprocessMemory* GetInterprocessMemory() const;

private:
	bool CleanupInterprocessObjects() const;
	void ReleaseInterprocessObjects();

public:
	const FString SharedMemoryRegionName;

private:
	void* PlatformMemoryRegion = nullptr;
};
