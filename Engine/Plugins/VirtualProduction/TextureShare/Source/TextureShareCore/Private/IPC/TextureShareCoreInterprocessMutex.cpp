// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPC/TextureShareCoreInterprocessMutex.h"
#include "Windows/WindowsPlatformProcess.h"
#include "Logging/LogScopedVerbosityOverride.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreInterprocessMutex
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareCoreInterprocessMutex::~FTextureShareCoreInterprocessMutex()
{
	ReleaseInterprocessMutex();
}

void FTextureShareCoreInterprocessMutex::ReleaseInterprocessMutex()
{
	if (PlatformMutex)
	{
		UnlockMutex();

		if (PlatformMutex)
		{
			FWindowsPlatformProcess::DeleteInterprocessSynchObject(PlatformMutex);

			PlatformMutex = nullptr;
		}
	}
}

bool FTextureShareCoreInterprocessMutex::InitializeInterprocessMutex(bool bInGlobalNameSpace, const FString& InMutexId)
{
	if (IsValid())
	{
		// Already initialized
		return false;
	}

	const FString FullMutexName = bInGlobalNameSpace
		? FString::Printf(TEXT("Global\\%s"), *InMutexId)
		: FString::Printf(TEXT("Local\\%s"), *InMutexId);

	FPlatformProcess::FSemaphore* ProcessMutex = nullptr;

	// In the global namespace, this mutex may already exist, so we should try to open it first.
	if (bInGlobalNameSpace)
	{
		// Try to open exist  mutex
		PlatformMutex = FPlatformProcess::NewInterprocessSynchObject(*FullMutexName, false);
	}

	// Otherwise try to create a new one.
	if (!PlatformMutex)
	{
		// Create new mutex
		PlatformMutex = FPlatformProcess::NewInterprocessSynchObject(*FullMutexName, true);
	}

	return IsValid();
}

bool FTextureShareCoreInterprocessMutex::LockMutex(const uint32 InMaxMillisecondsToWait)
{
	if (!IsValid())
	{
		return false;
	}

	// Converts miliseconds to the nanoseconds
	// 1ms = 10^6 ns
	const uint64 MaxNanosecondsToWait = InMaxMillisecondsToWait * 1000000ULL;

	// InMaxMillisecondsToWait=0 means a lock attempt without waiting.
	// Now we have never used an infinite lock to prevent deadlocks.
	return PlatformMutex->TryLock(MaxNanosecondsToWait);
}

void FTextureShareCoreInterprocessMutex::UnlockMutex()
{
	if (IsValid())
	{		
		// InMaxMillisecondsToWait=0 means a lock attempt without waiting.
		// Ensure it is locked before we try to unlock it
		PlatformMutex->TryLock(0);
		PlatformMutex->Unlock();
	}
}