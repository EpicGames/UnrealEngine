// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformProcess.h"

/**
 * The mutex class implementation.
 * This class can be used in both cases, for processes and threads.
 */
class FTextureShareCoreInterprocessMutex
{
public:
	FTextureShareCoreInterprocessMutex()
	{ }

	~FTextureShareCoreInterprocessMutex();

	/** Create a interprocess platform mutex object in the global namespace
	* @param InMutexId - the name of the interprocess mutex
	*/
	bool InitializeInterprocessMutex(const FString& InMutexId)
	{
		return InitializeInterprocessMutex(/** bInGlobalNameSpace= */ true, InMutexId);
	}

	/** Create a multithread platform mutex object in the local namespace. */
	bool Initialize()
	{
		const FString UniqueMutexName = FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensInBraces);

		return InitializeInterprocessMutex(/** bInGlobalNameSpace= */ false, UniqueMutexName);
	}

	/**
	* Tries to acquire and exclusive access for a specified amount of nanoseconds (also known as TryWait()).
	*
	* @param InMaxMillisecondsToWait - milliseconds to wait for
	* @return false if was not able to lock within given time
	*/
	bool LockMutex(const uint32 InMaxMillisecondsToWait);

	/** Relinquishes an exclusive access (also known as Release()) */
	void UnlockMutex();

	/** Is the platform mutex exist. */
	bool IsValid() const
	{
		return PlatformMutex != nullptr;
	}

private:
	/** Implementation of mutex opening/creation. */
	bool InitializeInterprocessMutex(bool bGlobalNameSpace, const FString& InMutexId);

	/** Implementation of mutex releasing. */
	void ReleaseInterprocessMutex();

private:
	// the reference to the platform mutex
	FGenericPlatformProcess::FSemaphore* PlatformMutex = nullptr;
};
