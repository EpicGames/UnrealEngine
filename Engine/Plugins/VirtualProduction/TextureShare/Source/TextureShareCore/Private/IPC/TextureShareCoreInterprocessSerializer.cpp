// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureShareCoreInterprocessSerializer.h"
#include "IPC/Containers/TextureShareCoreInterprocessObjectData.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreSerializeStreamRead
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareCoreSerializeStreamRead& FTextureShareCoreSerializeStreamRead::ReadDataFromStream(void* InDataPtr, const uint32_t InDataSize)
{
	// Read serialized data
	const uint32 TotalSize = SrcObject.DataHeader.Size;
	const uint32_t NextPos = CurrentPos + InDataSize;
	
	// check vs structure size diffs
	check(NextPos <= TotalSize);

	FPlatformMemory::Memcpy(InDataPtr, &SrcObject.DataMemory.Data[0] + CurrentPos, InDataSize);

	// Update current reading pos
	CurrentPos = NextPos;

	return *this;
}

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreSerializeStreamWrite
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareCoreSerializeStreamWrite& FTextureShareCoreSerializeStreamWrite::WriteDataToStream(const void* InDataPtr, const uint32_t InDataSize)
{
	const uint32 TotalSize = sizeof(FTextureShareCoreInterprocessObjectDataMemory);
	const uint32_t NextPos = CurrentPos + InDataSize;

	// check vs structure size diffs
	check(NextPos <= TotalSize);

	// Copy data to the shared memory
	FPlatformMemory::Memcpy(&DstObject.DataMemory.Data[0] + CurrentPos, InDataPtr, InDataSize);

	// udate current writing pos
	CurrentPos = NextPos;

	// Update shared memory region data size
	DstObject.DataHeader.Size = NextPos;

	return *this;
}
