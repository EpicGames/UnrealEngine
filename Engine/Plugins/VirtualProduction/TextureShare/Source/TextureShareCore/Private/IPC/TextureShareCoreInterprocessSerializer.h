// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/TextureShareCoreContainers.h"
#include "Core/Serialize/ITextureShareCoreSerializeStream.h"
#include "IPC/Containers/TextureShareCoreInterprocessObjectContainers.h"

struct FTextureShareCoreInterprocessObjectData;

/**
 * Simple memory serializer - Read
 */
class FTextureShareCoreSerializeStreamRead
	: public ITextureShareCoreSerializeStream
{
public:
	FTextureShareCoreSerializeStreamRead(const FTextureShareCoreInterprocessObjectData& InSrcObject)
		: SrcObject(InSrcObject)
	{ }

	virtual bool IsWriteStream() const
	{
		return false;
	}

protected:
	virtual FTextureShareCoreSerializeStreamRead& SerializeData(void* InDataPtr, const uint32_t InDataSize) override
	{
		return ReadDataFromStream(InDataPtr, InDataSize);
	}

private:
	FTextureShareCoreSerializeStreamRead& ReadDataFromStream(void* InDataPtr, const uint32_t InDataSize);

private:
	const FTextureShareCoreInterprocessObjectData& SrcObject;
	uint32_t CurrentPos = 0;
};

/**
 * Simple memory serializer - Write
 */
class FTextureShareCoreSerializeStreamWrite
	: public ITextureShareCoreSerializeStream
{
public:
	FTextureShareCoreSerializeStreamWrite(FTextureShareCoreInterprocessObjectData& InDstObject)
		: DstObject(InDstObject)
	{ }
	
	virtual bool IsWriteStream() const
	{
		return true;
	}

protected:
	virtual FTextureShareCoreSerializeStreamWrite& SerializeData(void* InDataPtr, const uint32_t InDataSize) override
	{
		return WriteDataToStream(InDataPtr, InDataSize);
	}

	virtual FTextureShareCoreSerializeStreamWrite& SerializeData(const void* InDataPtr, const uint32_t InDataSize) override
	{
		return WriteDataToStream(InDataPtr, InDataSize);
	}

protected:
	FTextureShareCoreSerializeStreamWrite& WriteDataToStream(const void* InDataPtr, const uint32_t InDataSize);

private:
	FTextureShareCoreInterprocessObjectData& DstObject;
	uint32_t CurrentPos = 0;
};
