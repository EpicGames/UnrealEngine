// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "ITextureShareCoreD3D11ResourcesCache.h"
#include "ITextureShareCoreD3D12ResourcesCache.h"
#include "ITextureShareCoreVulkanResourcesCache.h"

#include "Containers/TextureShareCoreContainers_ObjectDesc.h"

class ITextureShareCoreCallbacks;

/**
 * TextureShareCore API
 */
class TEXTURESHARECORE_API ITextureShareCoreAPI
{
public:
	virtual ~ITextureShareCoreAPI() = default;

public:
	/**
	 * Returns D3D11 TextureShare resources cache interface
	 */
	virtual TSharedPtr<ITextureShareCoreD3D11ResourcesCache, ESPMode::ThreadSafe> GetD3D11ResourcesCache() = 0;

	/**
	 * Returns D3D12 TextureShare resources cache interface
	 */
	virtual TSharedPtr<ITextureShareCoreD3D12ResourcesCache, ESPMode::ThreadSafe> GetD3D12ResourcesCache() = 0;

	/**
	 * Returns Vulkan TextureShare resources cache interface
	 */
	virtual TSharedPtr<ITextureShareCoreVulkanResourcesCache, ESPMode::ThreadSafe> GetVulkanResourcesCache() = 0;

public:
	/**
	 * Get or create new TextureShareCore object
	 *
	 * @param ShareName - Unique share name. Objects with the same name will try to connect
	 *
	 * @return - Ptr to TextureShareCore object
	 */
	virtual TSharedPtr<class ITextureShareCoreObject, ESPMode::ThreadSafe> GetOrCreateCoreObject(const FString& ShareName, const ETextureShareProcessType ProcessType = ETextureShareProcessType::Undefined) = 0;

	/**
	 * The test for the TextureShareCore object with the specified name exists.
	 *
	 * @param ShareName - Unique share name
	 *
	 * @return - true, if object with the specified name still exists.
	 */
	virtual bool IsCoreObjectExist(const FString& ShareName) const = 0;

	/**
	 * Remove TextureShareCore object
	 *
	 * @param ShareName - Unique share name
	 *
	 * @return True if the object with this name removed
	 */
	virtual bool RemoveCoreObject(const FString& ShareName) = 0;

public:
	/**
	 * Returns a list of handles to interprocess TextureShare objects with the specified name.
	 *
	 * @param InShareName            - (optional) name of the TextureShare object
	 * @param OutInterprocessObjects - Output array with handles to TextureShare objects
	 *
	 * @return true, if success
	 */
	virtual bool GetInterprocessObjects(const FString& InShareName, TArraySerializable<FTextureShareCoreObjectDesc>& OutInterprocessObjects) const = 0;

	/**
	 * Returns detailed information about the local process
	 */
	virtual const struct FTextureShareCoreObjectProcessDesc& GetProcessDesc() const = 0;

	/**
	 * Assign a unique name to the local process. The changes only affect newly created TextureShare objects.
	 *
	 * @param InProcessId - Custom local process name
	 */
	virtual void SetProcessName(const FString& InProcessId) = 0;

	/**
	 * Assign a device type to the local process. The changes only affect newly created TextureShare objects.
	 *
	 * @param InDeviceType - render device type
	 */
	virtual bool SetProcessDeviceType(const ETextureShareDeviceType InDeviceType) = 0;

	/**
	 * Release all resources last used more than specified timeout value
	 *
	 * @param InMilisecondsTimeout- Timeout to remove resource
	 */
	virtual void RemoveUnusedResources(const uint32 InMilisecondsTimeOut) = 0;

public:
	/**
	* Access to the TextureShareCore callbacks API
	*
	* @return Callbacks API
	*/
	virtual ITextureShareCoreCallbacks& GetCallbacks() = 0;
};
