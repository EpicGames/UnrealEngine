// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Containers/TextureShareCoreEnums.h"

class ITextureShareCoreObject;
class FRHICommandListImmediate;

/**
 * TextureShareCore callbacks API
 */
class ITextureShareCoreCallbacks
{
public:
	static ITextureShareCoreCallbacks& Get();

	virtual ~ITextureShareCoreCallbacks() = default;

public:
	/** Called on session start **/
	DECLARE_EVENT_OneParam(ITextureShareCoreCallbacks, FTextureShareCoreBeginSessionEvent, ITextureShareCoreObject&);
	virtual FTextureShareCoreBeginSessionEvent& OnTextureShareCoreBeginSession() = 0;

	/** Called on session end **/
	DECLARE_EVENT_OneParam(ITextureShareCoreCallbacks, FTextureShareCoreEndSessionEvent, ITextureShareCoreObject&);
	virtual FTextureShareCoreEndSessionEvent& OnTextureShareCoreEndSession() = 0;


	/** Called on begin frame sync **/
	DECLARE_EVENT_OneParam(ITextureShareCoreCallbacks, FTextureShareCoreBeginFrameSyncEvent, ITextureShareCoreObject&);
	virtual FTextureShareCoreBeginFrameSyncEvent& OnTextureShareCoreBeginFrameSync() = 0;

	/** Called on end frame sync **/
	DECLARE_EVENT_OneParam(ITextureShareCoreCallbacks, FTextureShareCoreEndFrameSyncEvent, ITextureShareCoreObject&);
	virtual FTextureShareCoreEndFrameSyncEvent& OnTextureShareCoreEndFrameSync() = 0;

	/** Called on frame sync **/
	DECLARE_EVENT_TwoParams(ITextureShareCoreCallbacks, FTextureShareCoreFrameSyncEvent, ITextureShareCoreObject&, const ETextureShareSyncStep);
	virtual FTextureShareCoreFrameSyncEvent& OnTextureShareCoreFrameSync() = 0;


	/** Called on begin frame sync on render thread **/
	DECLARE_EVENT_OneParam(ITextureShareCoreCallbacks, FTextureShareCoreBeginFrameSyncEvent_RenderThread, ITextureShareCoreObject&);
	virtual FTextureShareCoreBeginFrameSyncEvent_RenderThread& OnTextureShareCoreBeginFrameSync_RenderThread() = 0;

	/** Called on end frame sync on render thread **/
	DECLARE_EVENT_OneParam(ITextureShareCoreCallbacks, FTextureShareCoreEndFrameSyncEvent_RenderThread, ITextureShareCoreObject&);
	virtual FTextureShareCoreEndFrameSyncEvent_RenderThread& OnTextureShareCoreEndFrameSync_RenderThread() = 0;

	/** Called on frame sync on render thread **/
	DECLARE_EVENT_TwoParams(ITextureShareCoreCallbacks, FTextureShareCoreFrameSyncEvent_RenderThread, ITextureShareCoreObject&, const ETextureShareSyncStep);
	virtual FTextureShareCoreFrameSyncEvent_RenderThread& OnTextureShareCoreFrameSync_RenderThread() = 0;
};
