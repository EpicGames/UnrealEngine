// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Containers/TextureShareCoreEnums.h"
#include "Containers/TextureShareCoreContainers_DeviceD3D11.h"

struct FTextureShareCoreResourceHandle;
struct FTextureShareCoreResourceDesc;
struct FTextureShareCoreObjectDesc;

/**
 * API for Shared Resources on the D3D11 renderer
 */
class TEXTURESHARECORE_API ITextureShareCoreD3D11ResourcesCache
{
public:
	virtual ~ITextureShareCoreD3D11ResourcesCache() = default;

public:
	/**
	 * Create a handle to a share from a D3D11 resource
	 *
	 * @param InObjectDesc      - A handle to a TextureShare object. This object is the owner of the resource
	 * @param InD3D11Device     - D3D11 device interface
	 * @param InResourceD3D11   - D3D11 resource interface
	 * @param InResourceDesc    - Resource information for TS core (Eye, type of operation, sync pass, etc)
	 * @param OutResourceHandle - Output resource handle
	 *
	 * @return true if success
	 */
	virtual bool CreateSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D11Device* InD3D11Device, ID3D11Texture2D* InResourceD3D11, const FTextureShareCoreResourceDesc& InResourceDesc, FTextureShareCoreResourceHandle& OutResourceHandle) = 0;

	/**
	 * Open shared resource from the handle, and return D3D11 resource
	 *
	 * @param InObjectDesc     - A handle to a TextureShare object. This object is the owner of the resource
 	 * @param InD3D11Device    - D3D11 device interface
	 * @param InResourceHandle - Shared resource handle
	 *
	 * @return sharedPtr to D3D11 resource
	 */
	virtual ID3D11Texture2D* OpenSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D11Device* InD3D11Device, const FTextureShareCoreResourceHandle& InResourceHandle) = 0;

	/**
	 * Remove cached shared handle from D3D11 resource
	 * 
	 * @param InObjectDesc    - A handle to a TextureShare object. This object is the owner of the resource
	 * @param InResourceD3D11 - texture resource interface
	 *
	 * @return true if success
	 */
	virtual bool RemoveSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D11Texture2D* InResourceD3D11) = 0;
};
