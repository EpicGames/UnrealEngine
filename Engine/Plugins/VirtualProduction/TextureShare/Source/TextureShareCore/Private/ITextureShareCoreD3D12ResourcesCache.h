// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Containers/TextureShareCoreEnums.h"
#include "Containers/TextureShareCoreContainers_DeviceD3D12.h"

struct FTextureShareCoreResourceHandle;
struct FTextureShareCoreResourceDesc;
struct FTextureShareCoreObjectDesc;

/**
 * API for Shared Resources on the D3D12 renderer
 */
class TEXTURESHARECORE_API ITextureShareCoreD3D12ResourcesCache
{
public:
	virtual ~ITextureShareCoreD3D12ResourcesCache() = default;

public:
	/**
	 * Create a D3D12 cross-adapter resource
	 *
	 * @param InObjectDesc      - A handle to a TextureShare object. This object is the owner of the resource
	 * @param InD3D12Device     - D3D12 device interface
	 * @param InWidth           - resource size X
	 * @param InHeight          - resource size Y
	 * @param InFormat          - resource format
	 * @param InResourceDesc    - Resource information for TS core (Eye, type of operation, sync pass, etc)
	 * @param OutResourceHandle - Output resource handle
	 *
	 * @return nullptr if failed
	 */
	virtual ID3D12Resource* CreateCrossAdapterResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D12Device* InD3D12Device, const int32 InWidth, const int32 InHeight, const DXGI_FORMAT InFormat, const FTextureShareCoreResourceDesc& InResourceDesc, FTextureShareCoreResourceHandle& OutResourceHandle) = 0;

	/**
	 * Create a handle to a share from a D3D12 resource
	 *
	 * @param InObjectDesc      - A handle to a TextureShare object. This object is the owner of the resource
	 * @param InD3D12Device     - D3D12 device interface
	 * @param InResourceD3D12   - D3D12 resource interface
	 * @param InResourceDesc    - Resource information for TS core (Eye, type of operation, sync pass, etc)
	 * @param OutResourceHandle - Output resource handle
	 *
	 * @return true if success
	 */
	virtual bool CreateSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D12Device* InD3D12Device, ID3D12Resource* InResourceD3D12, const uint32 InResourceGPUIndex, const FTextureShareCoreResourceDesc& InResourceDesc, FTextureShareCoreResourceHandle& OutResourceHandle) = 0;

	/**
	 * Open shared resource from the handle, and return D3D12 resource
	 *
	 * @param InObjectDesc     - A handle to a TextureShare object. This object is the owner of the resource
	 * @param InD3D12Device    - D3D12 device interface
	 * @param InResourceHandle - Shared resource handle
	 *
	 * @return sharedPtr to D3D12 resource
	 */
	virtual ID3D12Resource* OpenSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D12Device* InD3D12Device, const FTextureShareCoreResourceHandle& InResourceHandle) = 0;

	/**
	 * Remove cached shared handle from D3D12 resource
	 *
	 * @param InObjectDesc    - A handle to a TextureShare object. This object is the owner of the resource
	 * @param InResourceD3D12 - texture resource interface
	 *
	 * @return true if success
	 */
	virtual bool RemoveSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, ID3D12Resource* InResourceD3D12) = 0;
};
