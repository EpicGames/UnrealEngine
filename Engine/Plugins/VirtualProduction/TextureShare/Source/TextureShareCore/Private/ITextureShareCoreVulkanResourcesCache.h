// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Containers/TextureShareCoreEnums.h"
#include "Containers/TextureShareCoreContainers_DeviceVulkan.h"

struct FTextureShareCoreResourceHandle;
struct FTextureShareCoreResourceDesc;
struct FTextureShareCoreObjectDesc;

/**
 * API for Shared Resources on the Vulkan renderer
 */
class TEXTURESHARECORE_API ITextureShareCoreVulkanResourcesCache
{
public:
	virtual ~ITextureShareCoreVulkanResourcesCache() = default;

public:
	/**
	 * Create a handle to a share from a Vulkan resource
	 *
	 * @param InObjectDesc          - A handle to a TextureShare object. This object is the owner of the resource
	 * @param InDeviceVulkanContext - Vulkan device context
	 * @param InVulkanResource      - Vulkan resource context
	 * @param InResourceDesc        - Resource information for TS core (Eye, type of operation, sync pass, etc)
	 * @param OutResourceHandle     - Output resource handle
	 *
	 * @return true if success
	 */
	virtual bool CreateSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, const FTextureShareDeviceVulkanContext& InDeviceVulkanContext, const FTextureShareDeviceVulkanResource& InVulkanResource, const FTextureShareCoreResourceDesc& InResourceDesc, FTextureShareCoreResourceHandle& OutResourceHandle) = 0;

	/**
	 * Open shared resource from the handle, and return Vulkan resource
	 *
	 * @param InObjectDesc          - A handle to a TextureShare object. This object is the owner of the resource
	 * @param InDeviceVulkanContext - Vulkan device context
	 * @param InResourceHandle      - Shared resource handle
	 *
	 * @return sharedPtr to Vulkan resource memory handle
	 */
	virtual FTextureShareDeviceVulkanResource OpenSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, const FTextureShareDeviceVulkanContext& InDeviceVulkanContext, const FTextureShareCoreResourceHandle& InResourceHandle) = 0;

	/**
	 * Remove cached shared handle from Vulkan resource
	 *
	 * @param InObjectDesc          - A handle to a TextureShare object. This object is the owner of the resource
	 * @param InDeviceVulkanContext - Vulkan device context
	 * @param InVulkanResource      - Vulkan resource context
	 *
	 * @return true if success
	 */
	virtual bool RemoveSharedResource(const FTextureShareCoreObjectDesc& InObjectDesc, const FTextureShareDeviceVulkanContext& InDeviceVulkanContext, const FTextureShareDeviceVulkanResource& InVulkanResource) = 0;
};
