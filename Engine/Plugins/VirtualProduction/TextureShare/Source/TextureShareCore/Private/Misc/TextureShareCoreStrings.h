// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if __UNREAL__
#include "CoreMinimal.h"
#else
#include "windows.h"
#endif

/**
 * Default names for TextureShareCore module
 */
namespace UE::TextureShareCoreStrings
{
	static constexpr auto DefaultShareName = TEXT("DefaultShareName");

	static constexpr auto DefaultViewId = TEXT("DefaultView");

	namespace DefaultProcessName
	{
#if defined TEXTURESHARE_PROJECT_NAME
		// Use TextureShare project name as process name
		static constexpr auto SDK = TEXT(TEXTURESHARE_PROJECT_NAME);
#else
		static constexpr auto SDK = TEXT("TextureShareSDK");
#endif
		static constexpr auto UE = TEXT("UnrealEngine");
	}
};
