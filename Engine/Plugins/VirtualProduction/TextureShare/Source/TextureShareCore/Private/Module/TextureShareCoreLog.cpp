// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/TextureShareCoreLog.h"

DEFINE_LOG_CATEGORY(LogTextureShareCore);
DEFINE_LOG_CATEGORY(LogTextureShareCoreD3D);
DEFINE_LOG_CATEGORY(LogTextureShareCoreWindows);
DEFINE_LOG_CATEGORY(LogTextureShareCoreObject);
DEFINE_LOG_CATEGORY(LogTextureShareCoreObjectSync);
DEFINE_LOG_CATEGORY(LogTextureShareCoreProxyObjectSync);

