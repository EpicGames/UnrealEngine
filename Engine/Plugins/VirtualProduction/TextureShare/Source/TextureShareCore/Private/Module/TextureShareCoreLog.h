// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TextureShareCoreLogDefines.h"

#if UE_BUILD_SHIPPING
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareCore, Warning, Warning);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareCoreD3D, Warning, Warning);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareCoreWindows, Warning, Warning);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareCoreObject, Warning, Warning);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareCoreObjectSync, Warning, Warning);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareCoreProxyObjectSync, Warning, Warning);
#else
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareCore, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareCoreD3D, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareCoreWindows, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareCoreObject, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareCoreObjectSync, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareCoreProxyObjectSync, Log, All);
#endif
