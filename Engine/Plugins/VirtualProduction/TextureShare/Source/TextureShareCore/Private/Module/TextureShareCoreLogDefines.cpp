// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/TextureShareCoreLogDefines.h"

#if TEXTURESHARECORE_SDK && TEXTURESHARECORE_DEBUGLOG

#include <windows.h>
#include "Debugapi.h"

//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreLogHelpers::WriteLine(const FString& In)
	{
		OutputDebugStringW(FTCHARToWChar(*FString::Printf(TEXT("%s\n\t"), *In)).Get());
	}

#endif
