// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if TEXTURESHARECORE_SDK==1

/**
 * SDK log helpers
 */

#ifdef UE_LOG
// SDK override UE log system
#undef UE_LOG
#endif

#if TEXTURESHARECORE_DEBUGLOG

//////////////////////////////////////////////////////////////////////////////////////////////
// Enable extra log in VS output window
struct FTextureShareCoreLogHelpers
{
	static void WriteLine(const FString& In);
};

#define UE_LOG(CategoryName, Verbosity, Format, ...)\
	FTextureShareCoreLogHelpers::WriteLine(FString::Printf(Format, ##__VA_ARGS__));

#define UE_TS_LOG(CategoryName, Verbosity, Format, ...)\
	FTextureShareCoreLogHelpers::WriteLine(FString::Printf(Format, ##__VA_ARGS__));

// Enable more extra log
#if TEXTURESHARECORE_BARRIER_DEBUGLOG

#define UE_TS_BARRIER_LOG(CategoryName, Verbosity, Format, ...)\
	FTextureShareCoreLogHelpers::WriteLine(FString::Printf(Format, ##__VA_ARGS__));

#else
// Disable extra log
#define UE_TS_BARRIER_LOG(CategoryName, Verbosity, Format, ...)
#endif

#else
// Disable all log
#define UE_LOG(CategoryName, Verbosity, Format, ...)
#define UE_TS_LOG(CategoryName, Verbosity, Format, ...)
#define UE_TS_BARRIER_LOG(CategoryName, Verbosity, Format, ...)

#endif

#endif

#if TEXTURESHARECORE_SDK==0

/**
 * UE log helpers
 */

// Enable extra log
#if TEXTURESHARECORE_DEBUGLOG

#define UE_TS_LOG(CategoryName, Verbosity, Format, ...)\
	UE_LOG(CategoryName, Verbosity, Format, ##__VA_ARGS__)

#else
// Disable extra log
#define UE_TS_LOG(CategoryName, Verbosity, Format, ...)
#endif

// Enable more extra log
#if TEXTURESHARECORE_BARRIER_DEBUGLOG

#define UE_TS_BARRIER_LOG(CategoryName, Verbosity, Format, ...)\
	UE_TS_LOG(CategoryName, Verbosity, Format, ##__VA_ARGS__)

#else
// Disable extra log
#define UE_TS_BARRIER_LOG(CategoryName, Verbosity, Format, ...)
#endif


#endif
