// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/TextureShareCoreModule.h"
#include "Module/TextureShareCoreLog.h"

#include "Core/TextureShareCore.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreModule
//////////////////////////////////////////////////////////////////////////////////////////////
ITextureShareCoreAPI& FTextureShareCoreModule::GetTextureShareCoreAPI()
{
	if (!TextureShareCoreAPI.IsValid())
	{
		TextureShareCoreAPI = MakeUnique<FTextureShareCore>();
		TextureShareCoreAPI->BeginSession();
	}

	return *TextureShareCoreAPI;
}

void FTextureShareCoreModule::StartupModule()
{
	UE_LOGF(LogTextureShareCore, Log, "TextureShareCore module startup");
}

void FTextureShareCoreModule::ShutdownModule()
{
	UE_LOGF(LogTextureShareCore, Log, "TextureShareCore module shutdown");

	ShutdownModuleImpl();
}

void FTextureShareCoreModule::ShutdownModuleImpl()
{
	if (TextureShareCoreAPI.IsValid())
	{
		TextureShareCoreAPI->EndSession();
		TextureShareCoreAPI.Reset();
	}
}

IMPLEMENT_MODULE(FTextureShareCoreModule, TextureShareCore);
