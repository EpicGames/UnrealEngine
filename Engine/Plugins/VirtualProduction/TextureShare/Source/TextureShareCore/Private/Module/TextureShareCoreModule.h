// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ITextureShareCore.h"
#include "Core/TextureShareCore.h"

/**
 * TextureShareCore module implementation
 */
class FTextureShareCoreModule
	: public ITextureShareCore
{
public:
	virtual ~FTextureShareCoreModule()
	{
		ShutdownModuleImpl();
	}

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

public:
	/**
	 * Returns Core API Interface
	 * Put headers in an Internal folder.
	 * This limits visibility to other engine modules, and allows implementations to be changed in hotfixes.
	 */
	virtual ITextureShareCoreAPI& GetTextureShareCoreAPI() override;

protected:
	void ShutdownModuleImpl();

private:
	TUniquePtr<FTextureShareCore> TextureShareCoreAPI;
};
