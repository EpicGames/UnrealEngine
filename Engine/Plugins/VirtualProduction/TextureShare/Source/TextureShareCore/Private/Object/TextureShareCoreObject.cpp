// Copyright Epic Games, Inc. All Rights Reserved.

#include "Object/TextureShareCoreObject.h"
#include "Object/TextureShareCoreObjectContainers.h"

#include "IPC/TextureShareCoreInterprocessMemoryRegion.h"
#include "IPC/TextureShareCoreInterprocessMutex.h"
#include "IPC/TextureShareCoreInterprocessEvent.h"
#include "IPC/Containers/TextureShareCoreInterprocessMemory.h"
#include "IPC/TextureShareCoreInterprocessHelpers.h"

#include "Core/TextureShareCore.h"
#include "Module/TextureShareCoreLog.h"

#include "Misc/ScopeLock.h"

namespace UE::TextureShareCore
{
	static FTextureShareCoreObjectDesc CreateNewObjectDesc(const FTextureShareCore& InOwner, const FString& InTextureShareName, const ETextureShareProcessType InProcessType)
	{
		FTextureShareCoreObjectDesc OutObjectDesc;

		// The name of this object
		OutObjectDesc.ShareName = InTextureShareName;

		// Generate unique Guid for each object
		OutObjectDesc.ObjectGuid = FGuid::NewGuid();

		// Get value for Owner
		OutObjectDesc.ProcessDesc = InOwner.GetProcessDesc();

		// Custom process type
		if (InProcessType != ETextureShareProcessType::Undefined)
		{
			OutObjectDesc.ProcessDesc.ProcessType = InProcessType;
		}

		return OutObjectDesc;
	}
};
using namespace UE::TextureShareCore;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreObject
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareCoreObject::FTextureShareCoreObject(FTextureShareCore& InOwner, const FString& InTextureShareName, const ETextureShareProcessType InProcessType)
	: ObjectDescMT(CreateNewObjectDesc(InOwner, InTextureShareName, InProcessType))
	, Owner(InOwner)
{
	NotificationEvent = Owner.CreateInterprocessEvent(ObjectDescMT.ObjectGuid);
}

FTextureShareCoreObject::~FTextureShareCoreObject()
{
	EndSession();
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreObject::HandleResetSync(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject)
{
#if TEXTURESHARECORE_DEBUGLOG
	UpdateFrameConnections(InterprocessMemory);
	UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:ResetSync(%s) %s"), *GetName(), *ToString(LocalObject), *ToString(GetFrameConnections()));
#endif

	// Reset GT+RT threads sync mutexes
	if (LockThreadMutex(ETextureShareThreadMutex::InternalLock))
	{
		// Special use case for these mutexes: ETextureShareThreadMutex::GameThread and ETextureShareThreadMutex::RenderingThread.
		// They used to lock threads (GT and RT), not data.
		// At this point, when we lose the sync, we should immediately reset all logic to the initial (unconnected) state.
		// This is a very specific use of the mutexes which don't follow the general rule of the lock()+unlock() pairs.
		UnlockThreadMutex(ETextureShareThreadMutex::GameThread);
		UnlockThreadMutex(ETextureShareThreadMutex::RenderingThread);

		UnlockThreadMutex(ETextureShareThreadMutex::InternalLock);
	}

	SetCurrentSyncStep(ETextureShareSyncStep::Undefined);

	LocalObject.Sync.ResetSync();

	SetFrameSyncState(ETextureShareCoreInterprocessObjectFrameSyncState::FrameSyncLost);

	// Reset current frame connections
	ResetFrameConnections();
}

//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreObject::SendNotificationEvents(const bool bInterprocessMemoryLockRequired)
{
	if (IsSessionActive() && IsActive())
	{
		if (!bInterprocessMemoryLockRequired || Owner.LockInterprocessMemory(GetTimeOutSettings().MemoryMutexTimeout))
		{
			if (FTextureShareCoreInterprocessMemory* InterprocessMemory = Owner.GetInterprocessMemory())
			{
				// Wake up all connectable processes
				TArray<const FTextureShareCoreInterprocessObject*> EventListeners;
				if (InterprocessMemory->FindObjectEventListeners(EventListeners, GetObjectDesc()))
				{
					for (const FTextureShareCoreInterprocessObject* RemoteObjectIt : EventListeners)
					{
						const FGuid EventGuid = RemoteObjectIt->Desc.ObjectGuid.ToGuid();

						if (!CachedNotificationEvents.Contains(EventGuid))
						{
							TSharedPtr<FEvent, ESPMode::ThreadSafe> Event = Owner.OpenInterprocessEvent(EventGuid);
							if (!Event.IsValid())
							{
								UE_LOGF(LogTextureShareCoreObject, Error, "TS'%ls': Can't open remote event from process '%ls'", *GetName(), *RemoteObjectIt->Desc.ProcessName.ToString());
							}

							CachedNotificationEvents.Emplace(EventGuid, Event);
						}

						if (CachedNotificationEvents[EventGuid].IsValid())
						{
							// Send wake up signal
							CachedNotificationEvents[EventGuid]->Trigger();
						}
					}
				}
			}

			if (bInterprocessMemoryLockRequired)
			{
				Owner.UnlockInterprocessMemory();
			}
		}
	}
}

void FTextureShareCoreObject::HandleFrameSkip(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject)
{
	UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:FrameSkip()"), *GetName());

	// Wake up remote processes anywait, because we change mem object header
	SendNotificationEvents(false);

	// Reset frame sync fro this frame
	HandleResetSync(InterprocessMemory, LocalObject);
}

void FTextureShareCoreObject::HandleFrameLost(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject)
{
	// Local frame connection lost
	UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:HandleFrameLost()"), *GetName());

	// Wake up remote processes anywait, because we change mem object header
	SendNotificationEvents(false);

	// Reset frame sync fro this frame
	HandleResetSync(InterprocessMemory, LocalObject);
}

bool FTextureShareCoreObject::TryWaitFrameProcesses(const uint32 InRemainMaxMillisecondsToWait)
{
	// Wake up remote processes anywait, because we change mem object header
	SendNotificationEvents(false);

	// Reset notification event
	NotificationEvent->Reset();

	// Unlock IPC shared memory
	Owner.UnlockInterprocessMemory();

	// Wait for remote processes data changed
	NotificationEvent->Wait(InRemainMaxMillisecondsToWait);

	// Try to lock shared memory again
	return Owner.LockInterprocessMemory(GetTimeOutSettings().MemoryMutexTimeout);
}

bool FTextureShareCoreObject::RemoveObject()
{
	return Owner.RemoveCoreObject(GetName());
}
