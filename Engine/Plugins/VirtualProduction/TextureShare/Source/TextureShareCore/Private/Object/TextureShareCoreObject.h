// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ITextureShareCoreObject.h"
#include "Core/TextureShareCore.h"

#include "IPC/Containers/TextureShareCoreInterprocessMemory.h"
#include "IPC/Containers/TextureShareCoreInterprocessObject.h"
#include "IPC/Containers/TextureShareCoreInterprocessEnums.h"

/**
 * TextureShareCore object api implementation
 */
class FTextureShareCoreObject
	: public ITextureShareCoreObject
{
public:
	FTextureShareCoreObject(FTextureShareCore& InOwner, const FString& InTextureShareName, const ETextureShareProcessType InProcessType);
	virtual ~FTextureShareCoreObject();

public:
	///////////////////// State /////////////////////

	virtual const FString& GetName() const override;

	virtual FTextureShareCoreObjectDesc GetObjectDesc() const override;
	virtual FTextureShareCoreObjectDesc GetObjectDesc_RenderThread() const override;

	virtual bool IsActive() const override;
	virtual bool IsActive_RenderThread() const override;

	virtual bool IsFrameSyncActive() const override;
	virtual bool IsFrameSyncActive_RenderThread() const override;

	virtual bool IsBeginFrameSyncActive() const override;
	virtual bool IsBeginFrameSyncActive_RenderThread() const override;

protected:
	const FTextureShareCoreObjectDesc GetObjectDesc_AnyThread() const;

private:
	// (MultiThread access data)
	FTextureShareCoreObjectDesc ObjectDescMT;

public:
	///////////////////// Settings /////////////////////

	virtual bool SetProcessId(const FString& InProcessId) override;
	virtual bool SetDeviceType(const ETextureShareDeviceType InDeviceType) override;

	virtual bool SetSyncSettings(const FTextureShareCoreSyncSettings& InSyncSettings) override;
	virtual FTextureShareCoreSyncSettings GetSyncSettings() const override;

	virtual FTextureShareCoreFrameSyncSettings GetFrameSyncSettings(const ETextureShareFrameSyncTemplate InType) const override;

private:
	// This settings used for a frame
	// (MultiThread access data)
	FTextureShareCoreSyncSettings SyncSettingsMT;

protected:
	void AddNewSyncStep(const ETextureShareSyncStep InSyncStep);
	void UpdateInterprocessObject();

	/** Special exception for timeout settings: they do not use arrays and are thread-safe. */
	const FTextureShareCoreTimeOutSettings& GetTimeOutSettings() const;

public:
	///////////////////// Session /////////////////////

	virtual bool BeginSession() override;
	virtual bool EndSession() override;
	virtual bool IsSessionActive() const override;

private:
	bool bSessionActive = false;

public:
	///////////////////// Thread sync support /////////////////////

	virtual bool LockThreadMutex(const ETextureShareThreadMutex InThreadMutex, bool bForceLockNoWait = false) const override;
	virtual bool UnlockThreadMutex(const ETextureShareThreadMutex InThreadMutex) const override;

protected:
	TSharedPtr<FTextureShareCoreInterprocessMutex, ESPMode::ThreadSafe> GetThreadMutex(const ETextureShareThreadMutex InThreadMutex) const;

	void InitializeThreadMutexes();
	void ReleaseThreadMutexes();

private:
	TArray<TSharedPtr<FTextureShareCoreInterprocessMutex, ESPMode::ThreadSafe>> ThreadMutexes;

public:
	///////////////////// Interprocess Synchronization /////////////////////

	virtual bool BeginFrameSync() override;
	virtual bool FrameSync(const ETextureShareSyncStep InSyncStep) override;
	virtual bool EndFrameSync() override;

	virtual bool BeginFrameSync_RenderThread() override;
	virtual bool FrameSync_RenderThread(const ETextureShareSyncStep InSyncStep) override;
	virtual bool EndFrameSync_RenderThread() override;

	virtual bool FindSkippedSyncStep(const ETextureShareSyncStep InSyncStep, ETextureShareSyncStep& OutSkippedSyncStep) override;
	virtual bool FindSkippedSyncStep_RenderThread(const ETextureShareSyncStep InSyncStep, ETextureShareSyncStep& OutSkippedSyncStep) override;

	virtual TArraySerializable<FTextureShareCoreObjectDesc> GetConnectedInterprocessObjects() const override;

protected:
	bool FindSkippedSyncStepImpl(const ETextureShareSyncStep InSyncStep, bool bIsProxyFrame, ETextureShareSyncStep& OutSkippedSyncStep);
	void ReleaseSyncData();

private:
	// When the frame starts, a list of processes connected to the frame is created.
	// New processes connected later inside this framelock will be ignored.
	// (MultiThread access data)
	TArraySerializable<FTextureShareCoreObjectDesc> FrameConnectionsMT;

	// Prevend double check for begion connection timeout
	bool bIsFrameConnectionTimeoutReached = false;

	ETextureShareCoreInterprocessObjectFrameSyncState FrameSyncState = ETextureShareCoreInterprocessObjectFrameSyncState::Undefined;
	ETextureShareSyncStep CurrentSyncStep = ETextureShareSyncStep::Undefined;

	// Local process notification event
	TSharedPtr<FEvent, ESPMode::ThreadSafe> NotificationEvent;

	// Opened events for remote processes
	TMap<FGuid, TSharedPtr<FEvent, ESPMode::ThreadSafe>> CachedNotificationEvents;

protected:
	bool TryFrameProcessesConnection(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject);
	bool ConnectFrameProcesses();
	bool DisconnectFrameProcesses();

	/** Return FrameConnectionsMT. */
	const TArraySerializable<FTextureShareCoreObjectDesc> GetFrameConnections() const;

	/** Return true is FrameConnectionsMT is empty. */
	bool IsEmptyFrameConnections() const;

	/** Reset FrameConnectionsMT array. */
	void ResetFrameConnections();

	/** Set FrameConnectionsMT array. */
	void SetFrameConnections(const TArraySerializable<FTextureShareCoreObjectDesc>& InNewFrameConnections);

	/** Update frame connections data from the IPC memory. */
	void UpdateFrameConnections(FTextureShareCoreInterprocessMemory& InterprocessMemory);

	/** Search for a frame connections. */
	int32 FindFrameConnections(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject);

	bool DoFrameSync(const ETextureShareSyncStep InSyncStep);
	bool DoFrameSync_RenderThread(const ETextureShareSyncStep InSyncStep);

	// Barrier
	bool BeginSyncBarrier(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject, const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass);
	bool AcceptSyncBarrier(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject, const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass);
	
	bool PrepareSyncBarrierPass(const ETextureShareSyncStep InSyncStep);
	bool PrepareSyncBarrierPass_RenderThread(const ETextureShareSyncStep InSyncStep);

	bool TryEnterSyncBarrier(const ETextureShareSyncStep InSyncStep) const;
	bool TryFrameProcessesBarrier(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject, const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass);

	// Wait until all connected processes entered to desired sync step
	bool SyncBarrierPass(const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass);
	bool SyncBarrierPass_RenderThread(const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass);

	bool SyncBarrierPassImpl(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject, const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass);

	void SetCurrentSyncStep(const ETextureShareSyncStep InSyncStep);
	void SetFrameSyncState(const ETextureShareCoreInterprocessObjectFrameSyncState InFrameSyncState);

	ETextureShareSyncStep FindNextSyncStep(const ETextureShareSyncStep InSyncStep) const;

	void UpdateLastAccessTime() const;

	bool TryWaitFrameProcesses(const uint32 InRemainMaxMillisecondsToWait);
	void SendNotificationEvents(const bool bInterprocessMemoryLockRequired);

	void HandleResetSync(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject);
	void HandleFrameLost(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject);
	void HandleFrameSkip(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject);

public:
	///////////////////// Data Containers /////////////////////

	virtual FTextureShareCoreData&      GetData() override;
	virtual FTextureShareCoreProxyData& GetProxyData_RenderThread() override;

	virtual TArraySerializable<FTextureShareCoreObjectData>      GetReceivedData() const override;
	virtual TArraySerializable<FTextureShareCoreObjectProxyData> GetReceivedProxyData_RenderThread() const override;

protected:
	void ReleaseData();

private:
	// Local process data
	FTextureShareCoreData      Data;
	FTextureShareCoreProxyData ProxyData;

	// Local copy of transfered resources and data
	TArraySerializable<FTextureShareCoreObjectData>      ReceivedObjectsData;
	TArraySerializable<FTextureShareCoreObjectProxyData> ReceivedObjectsProxyData;

public:
	///////////////////// Destructor /////////////////////

	virtual bool RemoveObject() override;

protected:
	// Data
	bool SendFrameData();
	bool ReceiveFrameData();

	bool SendFrameProxyData_RenderThread();
	bool ReceiveFrameProxyData_RenderThread();

private:
	FTextureShareCore& Owner;
};
