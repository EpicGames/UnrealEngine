// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Core/TextureShareCoreTime.h"

/**
 * Timer for synchronization
 */
struct FTextureShareCoreObjectTimeout
{
	FTextureShareCoreObjectTimeout(const uint32 InMaxMillisecondsToWait, const uint32 InMaxRemainMillisecondsToWait)
		: InitCycles64(FTextureShareCoreTime::Cycles64())
		, MaxCycles64ToWait(FTextureShareCoreTime::MilisecondsToCycles64(InMaxMillisecondsToWait))
		, MaxMillisecondsToWait(InMaxMillisecondsToWait)
		, MaxRemainMillisecondsToWait(InMaxRemainMillisecondsToWait)
	{ }

	uint64 GetLastCycles64() const
	{
		return InitCycles64 + MaxCycles64ToWait;
	}

	bool IsTimeOut() const
	{
		if (MaxMillisecondsToWait > 0)
		{
			return FTextureShareCoreTime::Cycles64() > GetLastCycles64();
		}

		// MaxMillisecondsToWait=0  for infinite
		return false;
	}

	uint32 GetRemainMaxMillisecondsToWait() const
	{
		if (MaxMillisecondsToWait > 0)
		{
			const uint64 LastCycles64    = GetLastCycles64();
			const uint64 CurrentCycles64 = FTextureShareCoreTime::Cycles64();

			if (CurrentCycles64 < LastCycles64)
			{
				if (uint32 Result = FTextureShareCoreTime::Cycles64ToMiliseconds(LastCycles64 - CurrentCycles64))
				{
					// Split wait multiplier, prevent hanging processes
					return FMath::Min(Result, MaxRemainMillisecondsToWait);
				}
			}

			// The values less than 1ms, rounded up
			return 1;
		}

		// Inifinite wait. Split to multiple, prevent stuck
		return MaxRemainMillisecondsToWait;
	}

private:
	const uint64 InitCycles64;
	const uint64 MaxCycles64ToWait;
	const uint32 MaxMillisecondsToWait;
	const uint32 MaxRemainMillisecondsToWait;
};
