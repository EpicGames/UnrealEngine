// Copyright Epic Games, Inc. All Rights Reserved.

#include "Object/TextureShareCoreObject.h"
#include "Object/TextureShareCoreObjectContainers.h"

#include "IPC/TextureShareCoreInterprocessMemoryRegion.h"
#include "IPC/TextureShareCoreInterprocessMutex.h"
#include "IPC/TextureShareCoreInterprocessEvent.h"
#include "IPC/Containers/TextureShareCoreInterprocessMemory.h"
#include "IPC/TextureShareCoreInterprocessHelpers.h"

#include "Module/TextureShareCoreModule.h"
#include "Module/TextureShareCoreLog.h"

#include "Misc/ScopeLock.h"

using namespace UE::TextureShareCore;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreObject
//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreObject::TryFrameProcessesConnection(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShareCore::TryFrameProcessesConnection(%s)"), *GetName()));

	// Mark last acces time for current process (this value used by other processes to detect die)
	LocalObject.Sync.UpdateLastAccessTime();

	// Collect valid processes to connect
	const int32 ReadyToConnectObjectsCount = FindFrameConnections(InterprocessMemory, LocalObject);

	UE_TS_BARRIER_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:TryFrameProcessesConnection(%s)=%d %s"), *GetName(), *ToString(LocalObject), ReadyToConnectObjectsCount, *ToString(GetFrameConnections()));

	// Handle sync logic
	const ETextureShareCoreFrameConnectionsState ConnectionsState = GetSyncSettings().FrameConnectionSettings.GetConnectionsState(ReadyToConnectObjectsCount, GetFrameConnections().Num());
	switch (ConnectionsState)
	{
	case ETextureShareCoreFrameConnectionsState::SkipFrame:
	default:
		// No available processes for connect. just skip this frame
		HandleFrameSkip(InterprocessMemory, LocalObject);

		// Reset first connect timeout when no processes
		bIsFrameConnectionTimeoutReached = false;

		// Break wait loop
		return false;

	case ETextureShareCoreFrameConnectionsState::Accept:

		// Reset first connect timeout after each success
		bIsFrameConnectionTimeoutReached = false;

		if (LocalObject.Sync.IsBarrierCompleted(ETextureShareSyncStep::InterprocessConnection, ETextureShareSyncPass::Enter))
		{
			SetFrameSyncState(ETextureShareCoreInterprocessObjectFrameSyncState::FrameConnected);

			// Break wait loop
			return false;
		}

		// Accept barrier
		AcceptSyncBarrier(InterprocessMemory, LocalObject, ETextureShareSyncStep::InterprocessConnection, ETextureShareSyncPass::Enter);

		// Continue this loop until EnterCompleted
		return true;

	case ETextureShareCoreFrameConnectionsState::Wait:
		if (bIsFrameConnectionTimeoutReached)
		{
			// after first timeout skip wait
			HandleFrameSkip(InterprocessMemory, LocalObject);
			return false;
		}

		// wait
		break;
	}

	// Reset connections list
	ResetFrameConnections();

	// Wait for new frame connection
	return true;
}

bool FTextureShareCoreObject::ConnectFrameProcesses()
{
	UE_TS_BARRIER_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:ConnectFrameProcesses()"), *GetName());

	ResetFrameConnections();

	if (IsSessionActive() && IsActive() && Owner.LockInterprocessMemory(GetTimeOutSettings().MemoryMutexTimeout))
	{
		if (FTextureShareCoreInterprocessMemory* InterprocessMemory = Owner.GetInterprocessMemory())
		{
			if (FTextureShareCoreInterprocessObject* LocalObject = InterprocessMemory->FindObject(GetObjectDesc()))
			{
				SetCurrentSyncStep(ETextureShareSyncStep::InterprocessConnection);
				SetFrameSyncState(ETextureShareCoreInterprocessObjectFrameSyncState::NewFrame);

				// Enter new frame sync barrier
				if (BeginSyncBarrier(*InterprocessMemory, *LocalObject, ETextureShareSyncStep::InterprocessConnection, ETextureShareSyncPass::Enter))
				{
					// Begin connect logic:
					FTextureShareCoreObjectTimeout FrameBeginTimer(GetTimeOutSettings().FrameBeginTimeOut, GetTimeOutSettings().FrameBeginTimeOutSplit);
					while (TryFrameProcessesConnection(*InterprocessMemory, *LocalObject))
					{
						if (FrameBeginTimer.IsTimeOut())
						{
							// Event error or timeout
							bIsFrameConnectionTimeoutReached = true;
							HandleFrameLost(*InterprocessMemory, *LocalObject);
							break;
						}

						// Wait for remote process data changes
						if (!TryWaitFrameProcesses(FrameBeginTimer.GetRemainMaxMillisecondsToWait()))
						{
							HandleFrameLost(*InterprocessMemory, *LocalObject);

							// Break this loop because we need to wake up remote processes and unlock IPC memory.
							break;
						}
					}
				}

				// There is no process to connect, reset sync
				if (IsEmptyFrameConnections() && FrameSyncState == ETextureShareCoreInterprocessObjectFrameSyncState::NewFrame)
				{
					// No process to connect, let's set the synchronization state as 'FrameSyncLost'.
					// This state is used in functions IsBeginFrameSyncActive() and IsBeginFrameSyncActive_RenderThread()
					SetFrameSyncState(ETextureShareCoreInterprocessObjectFrameSyncState::FrameSyncLost);
				}
			}
			else
			{
				UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:ConnectFrameProcesses: cant find local process desriptor in shared memory, GUID='%s'"), *GetName(), *GetObjectDesc().ObjectGuid.ToString(EGuidFormats::DigitsWithHyphens));
			}
		}
		else
		{
			UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:ConnectFrameProcesses Cant open Share memory"), *GetName());
		}

		// Wake up remote processes anywait, because we change mem object header
		SendNotificationEvents(false);

		Owner.UnlockInterprocessMemory();
	}

	if (!IsEmptyFrameConnections())
	{

		// Wait for other processes finish frame connect
		if (SyncBarrierPass(ETextureShareSyncStep::InterprocessConnection, ETextureShareSyncPass::Exit))
		{
			return true;
		}

		UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:ConnectFrameProcesses return FAILED (Exit barrier)"), *GetName());

		return false;
	}

	UE_TS_LOG(LogTextureShareCoreObjectSync, Warning, TEXT("%s:ConnectFrameProcesses - no processes"), *GetName());

	return false;
}

bool FTextureShareCoreObject::DisconnectFrameProcesses()
{
	return true;
}
