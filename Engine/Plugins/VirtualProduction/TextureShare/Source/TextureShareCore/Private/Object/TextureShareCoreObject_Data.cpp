// Copyright Epic Games, Inc. All Rights Reserved.

#include "Object/TextureShareCoreObject.h"
#include "Object/TextureShareCoreObjectContainers.h"

#include "IPC/TextureShareCoreInterprocessMemoryRegion.h"
#include "IPC/TextureShareCoreInterprocessMutex.h"
#include "IPC/TextureShareCoreInterprocessEvent.h"
#include "IPC/Containers/TextureShareCoreInterprocessMemory.h"
#include "IPC/Containers/TextureShareCoreInterprocessObjectContainers.h"
#include "IPC/TextureShareCoreInterprocessHelpers.h"

#include "Module/TextureShareCoreModule.h"
#include "Module/TextureShareCoreLog.h"

#include "Misc/ScopeLock.h"

using namespace UE::TextureShareCore;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreObject
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareCoreData& FTextureShareCoreObject::GetData()
{
	return Data;
}

FTextureShareCoreProxyData& FTextureShareCoreObject::GetProxyData_RenderThread()
{
	return ProxyData;
}

TArraySerializable<FTextureShareCoreObjectData> FTextureShareCoreObject::GetReceivedData() const
{
	return ReceivedObjectsData;
}

TArraySerializable<FTextureShareCoreObjectProxyData> FTextureShareCoreObject::GetReceivedProxyData_RenderThread() const
{
	return ReceivedObjectsProxyData;
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreObject::SendFrameData()
{
	bool bResult = false;
	if (IsFrameSyncActive() && Owner.LockInterprocessMemory(GetTimeOutSettings().MemoryMutexTimeout))
	{
		if (FTextureShareCoreInterprocessMemory* InterprocessMemory = Owner.GetInterprocessMemory())
		{
			// Get existing IPC object memory region
			if (FTextureShareCoreInterprocessObject* InterprocessObject = InterprocessMemory->FindObject(GetObjectDesc()))
			{
				FTextureShareCoreObjectDataRef FrameDataRef(GetObjectDesc(), Data);
				bResult = InterprocessObject->Data.Write(FrameDataRef);
			}
		}

		Owner.UnlockInterprocessMemory();
	}

	return bResult;
}

bool FTextureShareCoreObject::ReceiveFrameData()
{
	// Clear old data
	ReceivedObjectsData.Empty();

	bool bResult = false;
	if (IsFrameSyncActive() && Owner.LockInterprocessMemory(GetTimeOutSettings().MemoryMutexTimeout))
	{
		if (FTextureShareCoreInterprocessMemory* InterprocessMemory = Owner.GetInterprocessMemory())
		{
			bResult = true;
			// Read data from remote processes
			for (const FTextureShareCoreObjectDesc& RemoteObjDescIt : GetFrameConnections())
			{
				if (FTextureShareCoreInterprocessObject* InterprocessObject = InterprocessMemory->FindObject(RemoteObjDescIt))
				{
					if (InterprocessObject->Data.IsEnabled(ETextureShareCoreInterprocessObjectDataType::Frame))
					{
						ReceivedObjectsData.AddDefaulted();
						if (!InterprocessObject->Data.Read(ReceivedObjectsData.Last()))
						{
							ReceivedObjectsData.RemoveAt(ReceivedObjectsData.Num() - 1);
							bResult = false;
						}
					}
				}
			}
		}

		Owner.UnlockInterprocessMemory();
	}

	return bResult;
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreObject::SendFrameProxyData_RenderThread()
{
	bool bResult = false;
	if (IsFrameSyncActive_RenderThread() && Owner.LockInterprocessMemory(GetTimeOutSettings().MemoryMutexTimeout))
	{
		if (FTextureShareCoreInterprocessMemory* InterprocessMemory = Owner.GetInterprocessMemory())
		{
			// Get existing IPC object memory region
			if (FTextureShareCoreInterprocessObject* InterprocessObject = InterprocessMemory->FindObject(GetObjectDesc()))
			{
				FTextureShareCoreObjectProxyDataRef FrameProxyDataRef(GetObjectDesc(), ProxyData);
				bResult = InterprocessObject->Data.Write(FrameProxyDataRef);
			}
		}

		Owner.UnlockInterprocessMemory();
	}

	return bResult;
}

bool FTextureShareCoreObject::ReceiveFrameProxyData_RenderThread()
{
	// Clear old data
	ReceivedObjectsProxyData.Empty();

	bool bResult = false;
	if (IsFrameSyncActive_RenderThread() && Owner.LockInterprocessMemory(GetTimeOutSettings().MemoryMutexTimeout))
	{
		if (FTextureShareCoreInterprocessMemory* InterprocessMemory = Owner.GetInterprocessMemory())
		{
			bResult = true;
			// Read data from remote processes
			for (const FTextureShareCoreObjectDesc& RemoteObjDescIt : GetFrameConnections())
			{
				if (FTextureShareCoreInterprocessObject* InterprocessObject = InterprocessMemory->FindObject(RemoteObjDescIt))
				{
					if (InterprocessObject->Data.IsEnabled(ETextureShareCoreInterprocessObjectDataType::FrameProxy))
					{
						ReceivedObjectsProxyData.AddDefaulted();
						if (!InterprocessObject->Data.Read(ReceivedObjectsProxyData.Last()))
						{
							bResult = false;
							ReceivedObjectsProxyData.RemoveAt(ReceivedObjectsProxyData.Num() - 1);
						}
					}
				}
			}
		}

		Owner.UnlockInterprocessMemory();
	}

	return bResult;
}

void FTextureShareCoreObject::ReleaseData()
{
	UE_TS_LOG(LogTextureShareCoreObject, Log, TEXT("%s:ReleaseData()"), *GetName());

	Data.ResetData();
	ProxyData.ResetProxyData();
	ReceivedObjectsData.Empty();
	ReceivedObjectsProxyData.Empty();
}
