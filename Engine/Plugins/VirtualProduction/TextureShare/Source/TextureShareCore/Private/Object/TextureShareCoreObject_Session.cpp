// Copyright Epic Games, Inc. All Rights Reserved.

#include "Object/TextureShareCoreObject.h"
#include "Object/TextureShareCoreObjectContainers.h"

#include "IPC/TextureShareCoreInterprocessMemoryRegion.h"
#include "IPC/TextureShareCoreInterprocessMutex.h"
#include "IPC/TextureShareCoreInterprocessEvent.h"
#include "IPC/Containers/TextureShareCoreInterprocessMemory.h"
#include "IPC/TextureShareCoreInterprocessHelpers.h"

#include "Module/TextureShareCoreModule.h"
#include "Module/TextureShareCoreLog.h"

#include "ITextureShareCoreCallbacks.h"

#include "Misc/ScopeLock.h"

using namespace UE::TextureShareCore;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreObject
//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreObject::BeginSession()
{
	if (!IsSessionActive() && IsActive() && Owner.LockInterprocessMemory(GetTimeOutSettings().MemoryMutexTimeout))
	{
		const FTextureShareCoreSyncSettings CurrentSyncSettings = GetSyncSettings();

		// Update object desc sync hash value
		ObjectDescMT.Sync.SetSyncStepSettings(CurrentSyncSettings);

		if (FTextureShareCoreInterprocessMemory* InterprocessMemory = Owner.GetInterprocessMemory())
		{
			// Release dirty data for this share\this process
			InterprocessMemory->ReleaseDirtyObjects(GetObjectDesc());

			// Get ptr on a new IPC object memory region
			if (FTextureShareCoreInterprocessObject* InterprocessObject = InterprocessMemory->FindEmptyObject())
			{
				// Initialize IPC object data
				InterprocessObject->InitializeInterprocessObject(GetObjectDesc(), CurrentSyncSettings);

				InitializeThreadMutexes();

				// New IPC object created
				bSessionActive = true;

				if (ITextureShareCoreCallbacks::Get().OnTextureShareCoreBeginSession().IsBound())
				{
					ITextureShareCoreCallbacks::Get().OnTextureShareCoreBeginSession().Broadcast(*this);
				}

				UE_TS_LOG(LogTextureShareCoreObject, Log, TEXT("%s:BeginSession()"), *GetName());
			}
		}

		Owner.UnlockInterprocessMemory();
	}

	return bSessionActive;
}

bool FTextureShareCoreObject::EndSession()
{
	bool bResult = false;

	if (IsSessionActive() && IsActive())
	{
		bSessionActive = false;

		// Release IPC
		if (Owner.LockInterprocessMemory(GetTimeOutSettings().MemoryMutexTimeout))
		{
			if (FTextureShareCoreInterprocessMemory* InterprocessMemory = Owner.GetInterprocessMemory())
			{
				// Get existing IPC object memory region
				if (FTextureShareCoreInterprocessObject* InterprocessObject = InterprocessMemory->FindObject(GetObjectDesc()))
				{
					// Reset IPC object to defaults
					InterprocessObject->Release();

					if (ITextureShareCoreCallbacks::Get().OnTextureShareCoreEndSession().IsBound())
					{
						ITextureShareCoreCallbacks::Get().OnTextureShareCoreEndSession().Broadcast(*this);
					}

					UE_TS_LOG(LogTextureShareCoreObject, Log, TEXT("%s:EndSession()"), *GetName());

					bResult = true;
				}
			}

			Owner.UnlockInterprocessMemory();
		}

		// Release resources and handles
		Owner.RemoveCachedResources(GetObjectDesc());

		// Unlock and release thread mutexes
		ReleaseThreadMutexes();

		ReleaseSyncData();
		ReleaseData();
	}

	return bResult;
}

bool FTextureShareCoreObject::IsSessionActive() const
{
	return bSessionActive;
}
