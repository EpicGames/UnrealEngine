// Copyright Epic Games, Inc. All Rights Reserved.

#include "Object/TextureShareCoreObject.h"
#include "Object/TextureShareCoreObjectContainers.h"

#include "IPC/TextureShareCoreInterprocessMemoryRegion.h"
#include "IPC/TextureShareCoreInterprocessMutex.h"
#include "IPC/TextureShareCoreInterprocessEvent.h"
#include "IPC/Containers/TextureShareCoreInterprocessMemory.h"
#include "IPC/TextureShareCoreInterprocessHelpers.h"

#include "Module/TextureShareCoreModule.h"
#include "Module/TextureShareCoreLog.h"

#include "Misc/ScopeLock.h"

using namespace UE::TextureShareCore;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreObject
//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreObject::SetProcessId(const FString& InProcessId)
{
	if (InProcessId.IsEmpty())
	{
		// Skip equal or empty values
		return false;
	}

	bool bShouldSetProcessId = false;
	if (!IsSessionActive())
	{
		bShouldSetProcessId = (ObjectDescMT.ProcessDesc.ProcessId != InProcessId);
		if (bShouldSetProcessId)
		{
			ObjectDescMT.ProcessDesc.ProcessId = InProcessId;
		}
	}
	else if (LockThreadMutex(ETextureShareThreadMutex::InternalObjectDescMT))
	{
		bShouldSetProcessId = (ObjectDescMT.ProcessDesc.ProcessId != InProcessId);
		if (bShouldSetProcessId)
		{
			ObjectDescMT.ProcessDesc.ProcessId = InProcessId;
		}
		UnlockThreadMutex(ETextureShareThreadMutex::InternalObjectDescMT);
	}

	// Update IPC
	if (bShouldSetProcessId)
	{
		UpdateInterprocessObject();
	}

	if (bShouldSetProcessId)
	{
		UE_TS_LOG(LogTextureShareCoreObject, Log, TEXT("%s:SetProcessId('%s')"), *GetName(), *InProcessId);
	}

	return true;
}

bool FTextureShareCoreObject::SetDeviceType(const ETextureShareDeviceType InDeviceType)
{
	bool bShouldSetDeviceType = false;
	if (!IsSessionActive())
	{
		bShouldSetDeviceType = (ObjectDescMT.ProcessDesc.DeviceType != InDeviceType);
		if (bShouldSetDeviceType)
		{
			ObjectDescMT.ProcessDesc.DeviceType = InDeviceType;
		}
	}
	else if (LockThreadMutex(ETextureShareThreadMutex::InternalObjectDescMT))
	{
		bShouldSetDeviceType = (ObjectDescMT.ProcessDesc.DeviceType != InDeviceType);
		if (bShouldSetDeviceType)
		{
			ObjectDescMT.ProcessDesc.DeviceType = InDeviceType;
		}
		UnlockThreadMutex(ETextureShareThreadMutex::InternalObjectDescMT);
	}

	// Update IPC
	if (bShouldSetDeviceType)
	{
		UpdateInterprocessObject();
	}

	if (bShouldSetDeviceType)
	{
		UE_TS_LOG(LogTextureShareCoreObject, Log, TEXT("%s:SetDeviceType(%s)"), *GetName(), GetTEXT(InDeviceType));
	}

	return true;
}

//////////////////////////////////////////////////////////////////////////////////////////////
const FTextureShareCoreTimeOutSettings& FTextureShareCoreObject::GetTimeOutSettings() const
{
	// Special exception for timeout settings: they do not use arrays and are thread-safe.
	return SyncSettingsMT.TimeoutSettings;
}

FTextureShareCoreSyncSettings FTextureShareCoreObject::GetSyncSettings() const
{
	FTextureShareCoreSyncSettings OutSyncSettings;
	if (!IsSessionActive())
	{
		OutSyncSettings = SyncSettingsMT;
	}
	else if (LockThreadMutex(ETextureShareThreadMutex::InternalSyncSettingsMT))
	{
		OutSyncSettings = SyncSettingsMT;
		UnlockThreadMutex(ETextureShareThreadMutex::InternalSyncSettingsMT);
	}

	return OutSyncSettings;
}

bool FTextureShareCoreObject::SetSyncSettings(const FTextureShareCoreSyncSettings& InSyncSettings)
{
	bool bShouldUpdateSyncSettings = false;
	if (!IsSessionActive())
	{
		bShouldUpdateSyncSettings = SyncSettingsMT != InSyncSettings;
		SyncSettingsMT = InSyncSettings;
	}
	else if (LockThreadMutex(ETextureShareThreadMutex::InternalSyncSettingsMT))
	{
		bShouldUpdateSyncSettings = SyncSettingsMT != InSyncSettings;
		if (bShouldUpdateSyncSettings)
		{
			UE_TS_LOG(LogTextureShareCoreObject, Log, TEXT("%s:SetSyncSettings()"), *GetName());

			SyncSettingsMT = InSyncSettings;

			if (LockThreadMutex(ETextureShareThreadMutex::InternalObjectDescMT))
			{
				// Update object desc sync hash value
				ObjectDescMT.Sync.SetSyncStepSettings(SyncSettingsMT);
				UnlockThreadMutex(ETextureShareThreadMutex::InternalObjectDescMT);
			}
		}

		UnlockThreadMutex(ETextureShareThreadMutex::InternalSyncSettingsMT);
	}

	// Update IPC
	if (bShouldUpdateSyncSettings)
	{
		UpdateInterprocessObject();
	}

	return true;
}

void FTextureShareCoreObject::AddNewSyncStep(const ETextureShareSyncStep InSyncStep)
{
	bool bShouldUpdateSyncSettings = false;
	if (!IsSessionActive())
	{
		bShouldUpdateSyncSettings = !SyncSettingsMT.FrameSyncSettings.Steps.Contains(InSyncStep);
		SyncSettingsMT.FrameSyncSettings.Steps.AddSorted(InSyncStep);
	}
	else if (LockThreadMutex(ETextureShareThreadMutex::InternalSyncSettingsMT))
	{
		bShouldUpdateSyncSettings = !SyncSettingsMT.FrameSyncSettings.Steps.Contains(InSyncStep);
		if (bShouldUpdateSyncSettings)
		{
			UE_TS_LOG(LogTextureShareCoreObject, Log, TEXT("%s:AddNewSyncStep(%s)"), *GetName(), GetTEXT(InSyncStep));

			// Add requested sync step
			SyncSettingsMT.FrameSyncSettings.Steps.AddSorted(InSyncStep);

			if (LockThreadMutex(ETextureShareThreadMutex::InternalObjectDescMT))
			{
				// Update object desc sync hash value
				ObjectDescMT.Sync.SetSyncStepSettings(SyncSettingsMT);
				UnlockThreadMutex(ETextureShareThreadMutex::InternalObjectDescMT);
			}
		}
		UnlockThreadMutex(ETextureShareThreadMutex::InternalSyncSettingsMT);
	}

	// Update IPC
	if (bShouldUpdateSyncSettings)
	{
		UpdateInterprocessObject();
	}
}

void FTextureShareCoreObject::UpdateInterprocessObject()
{
	// Also update settings exposition for other processes
	if (IsSessionActive() && IsActive() && Owner.LockInterprocessMemory(GetTimeOutSettings().MemoryMutexTimeout))
	{
		if (FTextureShareCoreInterprocessMemory* InterprocessMemory = Owner.GetInterprocessMemory())
		{
			if (FTextureShareCoreInterprocessObject* LocalObject = InterprocessMemory->FindObject(GetObjectDesc()))
			{
				LocalObject->UpdateInterprocessObject(GetObjectDesc(), GetSyncSettings());
			}
		}

		Owner.UnlockInterprocessMemory();
	}

	UE_TS_LOG(LogTextureShareCoreObject, Log, TEXT("%s:UpdateInterprocessObject()"), *GetName());
}

FTextureShareCoreFrameSyncSettings FTextureShareCoreObject::GetFrameSyncSettings(const ETextureShareFrameSyncTemplate InType) const
{
	FTextureShareCoreFrameSyncSettings Result;
	{
		switch (InType)
		{
		case ETextureShareFrameSyncTemplate::Default:
			Result.Steps.Append({
				// GameThread logic
				ETextureShareSyncStep::FrameBegin,

					ETextureShareSyncStep::FramePreSetupBegin,

					ETextureShareSyncStep::FrameFlush,
				ETextureShareSyncStep::FrameEnd,

				// Proxy object sync settings (RenderingThread)
				ETextureShareSyncStep::FrameProxyBegin,

					ETextureShareSyncStep::FrameSceneFinalColorEnd,
					ETextureShareSyncStep::FrameProxyPreRenderEnd,
					ETextureShareSyncStep::FrameProxyBackBufferReadyToPresentEnd,

					ETextureShareSyncStep::FrameProxyFlush,
				ETextureShareSyncStep::FrameProxyEnd
				});
			break;

		case ETextureShareFrameSyncTemplate::SDK:
			Result.Steps.Append({
				// GameThread logic
				ETextureShareSyncStep::FrameBegin,

					// Synchronization steps are added upon request from the SDK

					ETextureShareSyncStep::FrameFlush,
				ETextureShareSyncStep::FrameEnd,

				// Proxy object sync settings (RenderingThread)
				ETextureShareSyncStep::FrameProxyBegin,

					// Synchronization steps are added upon request from the SDK

					ETextureShareSyncStep::FrameProxyFlush,
				ETextureShareSyncStep::FrameProxyEnd
				});
			break;

		case ETextureShareFrameSyncTemplate::DisplayCluster:
			Result.Steps.Append({
				// GameThread logic
				ETextureShareSyncStep::FrameBegin,

					ETextureShareSyncStep::FramePreSetupBegin,
					ETextureShareSyncStep::FrameSetupBegin,

					ETextureShareSyncStep::FrameFlush,
				ETextureShareSyncStep::FrameEnd,

				// Proxy object sync settings (RenderingThread)
				ETextureShareSyncStep::FrameProxyBegin,

					ETextureShareSyncStep::FrameProxyPreRenderEnd,
					ETextureShareSyncStep::FrameProxyRenderEnd,
					ETextureShareSyncStep::FrameProxyPostWarpEnd,
					ETextureShareSyncStep::FrameProxyPostRenderEnd,

					ETextureShareSyncStep::FrameProxyFlush,
				ETextureShareSyncStep::FrameProxyEnd
				});
			break;

		case ETextureShareFrameSyncTemplate::DisplayClusterCrossNode:
			Result.Steps.Append({
				// GameThread logic
				ETextureShareSyncStep::FrameBegin,
					// Synchronization steps are added upon request
					ETextureShareSyncStep::FrameFlush,
				ETextureShareSyncStep::FrameEnd,

				// Proxy object sync settings (RenderingThread)
				ETextureShareSyncStep::FrameProxyBegin,
					// Synchronization steps are added upon request
					ETextureShareSyncStep::FrameProxyFlush,
				ETextureShareSyncStep::FrameProxyEnd
				});
			break;

		default:
			UE_LOGF(LogTextureShareCoreObject, Error, "GetFrameSyncSettings: Not implemented for type '%ls'", GetTEXT(InType));
			break;
		}
	}

	return Result;
}
