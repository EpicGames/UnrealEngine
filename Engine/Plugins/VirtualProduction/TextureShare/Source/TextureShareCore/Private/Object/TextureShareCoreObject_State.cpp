// Copyright Epic Games, Inc. All Rights Reserved.

#include "Object/TextureShareCoreObject.h"
#include "Object/TextureShareCoreObjectContainers.h"

#include "IPC/TextureShareCoreInterprocessMemoryRegion.h"
#include "IPC/TextureShareCoreInterprocessMutex.h"
#include "IPC/TextureShareCoreInterprocessEvent.h"
#include "IPC/Containers/TextureShareCoreInterprocessMemory.h"
#include "IPC/TextureShareCoreInterprocessHelpers.h"

#include "Module/TextureShareCoreModule.h"
#include "Module/TextureShareCoreLog.h"

#include "Misc/ScopeLock.h"

using namespace UE::TextureShareCore;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreObject
//////////////////////////////////////////////////////////////////////////////////////////////
const FString& FTextureShareCoreObject::GetName() const
{
	return ObjectDescMT.ShareName;
}

const FTextureShareCoreObjectDesc FTextureShareCoreObject::GetObjectDesc_AnyThread() const
{
	FTextureShareCoreObjectDesc OutObjectDesc;
	if (!IsSessionActive())
	{
		OutObjectDesc = ObjectDescMT;
	}
	else if (LockThreadMutex(ETextureShareThreadMutex::InternalObjectDescMT))
	{
		OutObjectDesc = ObjectDescMT;
		UnlockThreadMutex(ETextureShareThreadMutex::InternalObjectDescMT);
	}

	return OutObjectDesc;
}

FTextureShareCoreObjectDesc FTextureShareCoreObject::GetObjectDesc() const
{
	return GetObjectDesc_AnyThread();
}

FTextureShareCoreObjectDesc FTextureShareCoreObject::GetObjectDesc_RenderThread() const
{
	return GetObjectDesc_AnyThread();
}
//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreObject::IsActive() const
{
	return NotificationEvent.IsValid() && Owner.IsActive();
}

bool FTextureShareCoreObject::IsActive_RenderThread() const
{
	return NotificationEvent.IsValid() && Owner.IsActive();
}

bool FTextureShareCoreObject::IsFrameSyncActive() const
{
	return IsSessionActive() && IsActive() && !IsEmptyFrameConnections();
}

bool FTextureShareCoreObject::IsFrameSyncActive_RenderThread() const
{
	return IsSessionActive() && IsActive_RenderThread() && !IsEmptyFrameConnections();
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreObject::IsBeginFrameSyncActive() const
{
	switch (FrameSyncState)
	{
	case ETextureShareCoreInterprocessObjectFrameSyncState::FrameProxyBegin:
	case ETextureShareCoreInterprocessObjectFrameSyncState::FrameProxyEnd:
	case ETextureShareCoreInterprocessObjectFrameSyncState::FrameEnd:
	case ETextureShareCoreInterprocessObjectFrameSyncState::Undefined:
	case ETextureShareCoreInterprocessObjectFrameSyncState::FrameSyncLost:
		break;

	default:
		// logic broken
		UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:BeginFrameSync() - frame logic broken = %s"), *GetName(), GetTEXT(FrameSyncState));

		return false;
	}

	return true;
}

bool FTextureShareCoreObject::IsBeginFrameSyncActive_RenderThread() const
{
	if (IsEmptyFrameConnections())
	{
		UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Log, TEXT("%s:IsBeginFrameSyncActive_RenderThread() - canceled: no connections"), *GetName());

		return false;
	}

	switch (FrameSyncState)
	{
	case ETextureShareCoreInterprocessObjectFrameSyncState::FrameProxyBegin:
	case ETextureShareCoreInterprocessObjectFrameSyncState::FrameProxyEnd:
		// logic broken
		UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Error, TEXT("%s:BeginFrameProxySync() - frame logic broken = %s"), *GetName(), GetTEXT(FrameSyncState));

		return false;

	default:
		break;
	}

	return true;
}
