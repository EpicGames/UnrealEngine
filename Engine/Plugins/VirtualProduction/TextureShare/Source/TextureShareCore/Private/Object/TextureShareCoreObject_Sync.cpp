// Copyright Epic Games, Inc. All Rights Reserved.

#include "Object/TextureShareCoreObject.h"
#include "Object/TextureShareCoreObjectContainers.h"

#include "IPC/TextureShareCoreInterprocessMemoryRegion.h"
#include "IPC/TextureShareCoreInterprocessMutex.h"
#include "IPC/TextureShareCoreInterprocessEvent.h"
#include "IPC/Containers/TextureShareCoreInterprocessMemory.h"
#include "IPC/TextureShareCoreInterprocessHelpers.h"

#include "Module/TextureShareCoreModule.h"
#include "Module/TextureShareCoreLog.h"

#include "Misc/ScopeLock.h"

#include "ITextureShareCoreCallbacks.h"

using namespace UE::TextureShareCore;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreObject
//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreObject::FindSkippedSyncStep(const ETextureShareSyncStep InSyncStep, ETextureShareSyncStep& OutSkippedSyncStep)
{
	return FindSkippedSyncStepImpl(InSyncStep, false, OutSkippedSyncStep);
}

bool FTextureShareCoreObject::FindSkippedSyncStep_RenderThread(const ETextureShareSyncStep InSyncStep, ETextureShareSyncStep& OutSkippedSyncStep)
{
	return FindSkippedSyncStepImpl(InSyncStep, true, OutSkippedSyncStep);
}

bool FTextureShareCoreObject::FindSkippedSyncStepImpl(const ETextureShareSyncStep InSyncStep, bool bIsProxyFrame, ETextureShareSyncStep& OutSkippedSyncStep)
{
	// Update sync steps on request
	AddNewSyncStep(InSyncStep);

	const TArraySerializable<ETextureShareSyncStep> SyncSteps = GetSyncSettings().FrameSyncSettings.Steps;
	if(const int32 NextStepIndex = SyncSteps.Find(CurrentSyncStep) + 1)
	{
		const ETextureShareSyncStep FrameEndSyncStep = bIsProxyFrame ? ETextureShareSyncStep::FrameProxyEnd : ETextureShareSyncStep::FrameEnd;

		OutSkippedSyncStep = SyncSteps[NextStepIndex];
		if (OutSkippedSyncStep < InSyncStep && OutSkippedSyncStep < FrameEndSyncStep)
		{
			return true;
		}
	}

	return false;
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreObject::FrameSync(const ETextureShareSyncStep InSyncStep)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShareCore::FrameSync(%s, %s)"), *GetName(), GetTEXT(InSyncStep)));

	if (!IsFrameSyncActive())
	{
		UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:FrameSync(%s) failed: no sync for this frame"), *GetName(), GetTEXT(InSyncStep));

		return false;
	}

	if (CurrentSyncStep == InSyncStep)
	{
		return true;
	}

	UE_TS_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:FrameSync(%s)"), *GetName(), GetTEXT(InSyncStep));

	// Recall all skipped sync steps
	ETextureShareSyncStep SkippedSyncStep;
	while (FindSkippedSyncStep(InSyncStep, SkippedSyncStep))
	{
		if (!DoFrameSync(SkippedSyncStep))
		{
			UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:FrameSync(%s) failed handle skipped syncstep '%s'"), *GetName(), GetTEXT(InSyncStep), GetTEXT(SkippedSyncStep));

			return false;
		}
	}

	// Call requested syncstep
	if (!DoFrameSync(InSyncStep))
	{
		UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:FrameSync(%s) failed"), *GetName(), GetTEXT(InSyncStep));

		return false;
	}

	return true;
}

bool FTextureShareCoreObject::FrameSync_RenderThread(const ETextureShareSyncStep InSyncStep)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShareCore::FrameSync_RenderThread(%s, %s)"), *GetName(), GetTEXT(InSyncStep)));

	if (!IsFrameSyncActive_RenderThread())
	{
		UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Error, TEXT("%s:FrameSync_RenderThread(%s) failed: no sync for this frame"), *GetName(), GetTEXT(InSyncStep));

		return false;
	}

	if (CurrentSyncStep == InSyncStep)
	{
		// Skip duplicated calls
		return true;
	}
	
	UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Log, TEXT("%s:FrameSync_RenderThread(%s)"), *GetName(), GetTEXT(InSyncStep));

	// Recall all skipped sync steps
	ETextureShareSyncStep SkippedSyncStep;
	while (FindSkippedSyncStep(InSyncStep, SkippedSyncStep))
	{
		if (!DoFrameSync_RenderThread(SkippedSyncStep))
		{
			UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Error, TEXT("%s:FrameSync_RenderThread(%s) failed handle skipped syncstep '%s'"), *GetName(), GetTEXT(InSyncStep), GetTEXT(SkippedSyncStep));

			return false;
		}
	}

	// call requested syncstep
	if (!DoFrameSync_RenderThread(InSyncStep))
	{
		UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Error, TEXT("%s:FrameSync_RenderThread(%s) failed"), *GetName(), GetTEXT(InSyncStep));

		return false;
	}

	return true;
}

bool FTextureShareCoreObject::BeginFrameSync()
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShareCore::BeginFrameSync(%s)"), *GetName()));

	UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Log, TEXT("%s:BeginFrameSync()"), *GetName());

	if (!IsBeginFrameSyncActive())
	{
		return false;
	}

	// Reset prev-frame data
	Data.ResetData();

	// And connect new frame processes (updates every frame)
	if (!ConnectFrameProcesses())
	{
		UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:BeginFrameSync() failed"), *GetName());

		return false;
	}

	SetCurrentSyncStep(ETextureShareSyncStep::FrameBegin);
	SetFrameSyncState(ETextureShareCoreInterprocessObjectFrameSyncState::FrameBegin);

	return true;
}

bool FTextureShareCoreObject::BeginFrameSync_RenderThread()
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShareCore::BeginFrameSync_RenderThread(%s)"), *GetName()));

	UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Log, TEXT("%s:BeginFrameSync_RenderThread()"), *GetName());

	if (!IsBeginFrameSyncActive_RenderThread())
	{
		return false;
	}

	if (!IsFrameSyncActive_RenderThread())
	{
		UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Error, TEXT("%s:BeginFrameSync_RenderThread() failed: no sync for this frame"), *GetName());

		return false;
	}

	// Reset prev-frame proxy data
	ProxyData.ResetProxyData();

	SetCurrentSyncStep(ETextureShareSyncStep::FrameProxyBegin);
	SetFrameSyncState(ETextureShareCoreInterprocessObjectFrameSyncState::FrameProxyBegin);

	return true;
}

bool FTextureShareCoreObject::EndFrameSync()
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShareCore::EndFrameSync(%s)"), *GetName()));

	if (!IsFrameSyncActive())
	{
		UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:EndFrameSync() failed: no sync for this frame"), *GetName());

		return false;
	}

	UE_TS_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:EndFrameSync()"), *GetName());

	// Always force flush sync
	if (!FrameSync(ETextureShareSyncStep::FrameFlush) ||
		FrameSyncState != ETextureShareCoreInterprocessObjectFrameSyncState::FrameBegin)
	{
		UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:EndFrameSync() - failed"), *GetName());

		return false;
	}

	SetCurrentSyncStep(ETextureShareSyncStep::FrameEnd);
	SetFrameSyncState(ETextureShareCoreInterprocessObjectFrameSyncState::FrameEnd);

	return true;
}

bool FTextureShareCoreObject::EndFrameSync_RenderThread()
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShareCore::EndFrameSync_RenderThread(%s)"), *GetName()));

	if (!IsFrameSyncActive_RenderThread())
	{
		UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Error, TEXT("%s:EndFrameSync_RenderThread() failed: no sync for this frame"), *GetName());

		return false;
	}

	UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Log, TEXT("%s:EndFrameSync_RenderThread()"), *GetName());

	// Always force flush sync
	if(!FrameSync_RenderThread(ETextureShareSyncStep::FrameProxyFlush)
	|| FrameSyncState != ETextureShareCoreInterprocessObjectFrameSyncState::FrameProxyBegin)
	{
		UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Error, TEXT("%s:EndFrameSync_RenderThread() - failed"), *GetName());

		return false;
	}

	// And finally disconnect frame processes
	if (!DisconnectFrameProcesses())
	{
		UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Error, TEXT("%s:EndFrameSync_RenderThread() DisconnectFrameProcesses failed"), *GetName());

		return false;
	}

	SetCurrentSyncStep(ETextureShareSyncStep::FrameProxyEnd);
	SetFrameSyncState(ETextureShareCoreInterprocessObjectFrameSyncState::FrameProxyEnd);

	return true;
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreObject::DoFrameSync(const ETextureShareSyncStep InSyncStep)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShareCore::DoFrameSync(%s, %s)"), *GetName(), GetTEXT(InSyncStep)));

	if (InSyncStep == ETextureShareSyncStep::FrameFlush)
	{
		// Always skip special flush sync pass
		return true;
	}

	if (!IsFrameSyncActive())
	{
		UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:DoFrameSync(%s) - disabled sync"), *GetName(), GetTEXT(InSyncStep));

		return false;
	}

	if (!TryEnterSyncBarrier(InSyncStep))
	{
		UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:DoFrameSync(%s) - failed entering to barriers"), *GetName(), GetTEXT(InSyncStep));

		return false;
	}

	if (!PrepareSyncBarrierPass(InSyncStep))
	{
		// Skip this sync step - other processes not support
		SetCurrentSyncStep(InSyncStep);
		UE_TS_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:DoFrameSync(%s) - Skipped"), *GetName(), GetTEXT(InSyncStep));

		return true;
	}

	UE_TS_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:DoFrameSync(%s)"), *GetName(), GetTEXT(InSyncStep));

	// Write local data to the shared memory
	SendFrameData();

	// Use 2 barriers to exchange data between all processes
	if (!SyncBarrierPass(InSyncStep, ETextureShareSyncPass::Enter))
	{
		UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:DoFrameSync(%s) - Enter barrier failed"), *GetName(), GetTEXT(InSyncStep));

		return false;
	}

	// Receive frame data from the connected processes
	ReceiveFrameData();

	// Entering to the new sync step at the moment
	SetCurrentSyncStep(InSyncStep);

	// Exit from current sync step barrier
	if(!SyncBarrierPass(InSyncStep, ETextureShareSyncPass::Exit))
	{
		UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:DoFrameSync(%s) - Exit barrier failed"), *GetName(), GetTEXT(InSyncStep));

		return false;
	}

	return true;
}

bool FTextureShareCoreObject::DoFrameSync_RenderThread(const ETextureShareSyncStep InSyncStep)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("TextureShareCore::DoFrameSync_RenderThread(%s, %s)"), *GetName(), GetTEXT(InSyncStep)));

	if (InSyncStep == ETextureShareSyncStep::FrameProxyFlush)
	{
		// Always skip special flush sync pass
		return true;
	}

	if (!IsFrameSyncActive_RenderThread())
	{
		UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Error, TEXT("%s:FrameSync_RenderThread(%s) - disabled sync"), *GetName(), GetTEXT(InSyncStep));

		return false;
	}

	if (!TryEnterSyncBarrier(InSyncStep))
	{
		UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Error, TEXT("%s:FrameSync_RenderThread(%s)  - failed entering to barriers"), *GetName(), GetTEXT(InSyncStep));

		return false;
	}

	if (!PrepareSyncBarrierPass_RenderThread(InSyncStep))
	{
		UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Log, TEXT("%s:FrameSync_RenderThread(%s) - Skipped"), *GetName(), GetTEXT(InSyncStep));

		SetCurrentSyncStep(InSyncStep);

		// Skip this sync step - other processes not support
		return true;
	}

	UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Log, TEXT("%s:DoFrameSync_RenderThread(%s)"), *GetName(), GetTEXT(InSyncStep));
	
	// Write local frame proxy data to the shared memory
	SendFrameProxyData_RenderThread();

	// Use 2 barriers to exchange data between all processes
	if (!SyncBarrierPass_RenderThread(InSyncStep, ETextureShareSyncPass::Enter))
	{
		UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Error, TEXT("%s:DoFrameSync_RenderThread(%s) - Enter barrier failed"), *GetName(), GetTEXT(InSyncStep));

		return false;
	}

	// Receive proxy frame data from the connected processes
	ReceiveFrameProxyData_RenderThread();

	// Entering to the new sync step at the moment
	SetCurrentSyncStep(InSyncStep);

	// Exit from current sync step barrier
	if (!SyncBarrierPass_RenderThread(InSyncStep, ETextureShareSyncPass::Exit))
	{
		UE_TS_LOG(LogTextureShareCoreProxyObjectSync, Error, TEXT("%s:DoFrameSync_RenderThread(%s) - Exit barrier failed"), *GetName(), GetTEXT(InSyncStep));

		return false;
	}

	return true;
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreObject::SetCurrentSyncStep(const ETextureShareSyncStep InCurrentSyncStep)
{
	UE_TS_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:SetCurrentSyncStep(%s->%s)"), *GetName(), GetTEXT(CurrentSyncStep), GetTEXT(InCurrentSyncStep));

	CurrentSyncStep = InCurrentSyncStep;
}

void FTextureShareCoreObject::SetFrameSyncState(const ETextureShareCoreInterprocessObjectFrameSyncState InFrameSyncState)
{
	UE_TS_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:SetFrameSyncState(%s->%s)"), *GetName(), GetTEXT(FrameSyncState), GetTEXT(InFrameSyncState));

	FrameSyncState = InFrameSyncState;
}

//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreObject::UpdateLastAccessTime() const
{
	if (IsSessionActive() && IsActive() && Owner.LockInterprocessMemory(GetTimeOutSettings().MemoryMutexTimeout))
	{
		if (FTextureShareCoreInterprocessMemory* InterprocessMemory = Owner.GetInterprocessMemory())
		{
			// Get existing IPC object memory region
			if (FTextureShareCoreInterprocessObject* InterprocessObject = InterprocessMemory->FindObject(GetObjectDesc()))
			{
				InterprocessObject->Sync.UpdateLastAccessTime();
			}
		}

		Owner.UnlockInterprocessMemory();
	}
}

void FTextureShareCoreObject::ReleaseSyncData()
{
	UE_TS_LOG(LogTextureShareCoreObject, Log, TEXT("%s:ReleaseSyncData()"), *GetName());

	ResetFrameConnections();

	FrameSyncState = ETextureShareCoreInterprocessObjectFrameSyncState::Undefined;
	CurrentSyncStep = ETextureShareSyncStep::Undefined;

	CachedNotificationEvents.Empty();
}

//////////////////////////////////////////////////////////////////////////////////////////////
TArraySerializable<FTextureShareCoreObjectDesc> FTextureShareCoreObject::GetConnectedInterprocessObjects() const
{
	return GetFrameConnections();
}

void FTextureShareCoreObject::UpdateFrameConnections(FTextureShareCoreInterprocessMemory& InterprocessMemory)
{
	if (LockThreadMutex(ETextureShareThreadMutex::InternalFrameConnectionsMT))
	{
		InterprocessMemory.UpdateFrameConnections(FrameConnectionsMT);

		UnlockThreadMutex(ETextureShareThreadMutex::InternalFrameConnectionsMT);
	}
}

int32 FTextureShareCoreObject::FindFrameConnections(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject)
{
	int32 ProcessNum = 0;
	if (LockThreadMutex(ETextureShareThreadMutex::InternalFrameConnectionsMT))
	{
		ProcessNum = InterprocessMemory.FindConnectableObjects(FrameConnectionsMT, LocalObject);

		UnlockThreadMutex(ETextureShareThreadMutex::InternalFrameConnectionsMT);
	}

	return ProcessNum;
}

void FTextureShareCoreObject::SetFrameConnections(const TArraySerializable<FTextureShareCoreObjectDesc>& InNewFrameConnections)
{
	if (LockThreadMutex(ETextureShareThreadMutex::InternalFrameConnectionsMT))
	{
		FrameConnectionsMT = InNewFrameConnections;

		UnlockThreadMutex(ETextureShareThreadMutex::InternalFrameConnectionsMT);
	}
}

void FTextureShareCoreObject::ResetFrameConnections()
{
	if (!IsSessionActive())
	{
		FrameConnectionsMT.Reset();
	}
	else if (LockThreadMutex(ETextureShareThreadMutex::InternalFrameConnectionsMT))
	{
		FrameConnectionsMT.Reset();

		UnlockThreadMutex(ETextureShareThreadMutex::InternalFrameConnectionsMT);
	}
}

bool FTextureShareCoreObject::IsEmptyFrameConnections() const
{
	bool bIsEmptyFrameConnections = true;
	if (LockThreadMutex(ETextureShareThreadMutex::InternalFrameConnectionsMT))
	{
		bIsEmptyFrameConnections = FrameConnectionsMT.IsEmpty();

		UnlockThreadMutex(ETextureShareThreadMutex::InternalFrameConnectionsMT);
	}
	
	return bIsEmptyFrameConnections;
}

const TArraySerializable<FTextureShareCoreObjectDesc> FTextureShareCoreObject::GetFrameConnections() const
{
	TArraySerializable<FTextureShareCoreObjectDesc> OutFrameConnections;
	if (!IsSessionActive())
	{
		OutFrameConnections = FrameConnectionsMT;
	}
	else  if (LockThreadMutex(ETextureShareThreadMutex::InternalFrameConnectionsMT))
	{
		OutFrameConnections = FrameConnectionsMT;

		UnlockThreadMutex(ETextureShareThreadMutex::InternalFrameConnectionsMT);
	}

	return OutFrameConnections;
}