// Copyright Epic Games, Inc. All Rights Reserved.

#include "Object/TextureShareCoreObject.h"
#include "Object/TextureShareCoreObjectContainers.h"

#include "IPC/TextureShareCoreInterprocessMemoryRegion.h"
#include "IPC/TextureShareCoreInterprocessMutex.h"
#include "IPC/TextureShareCoreInterprocessEvent.h"
#include "IPC/Containers/TextureShareCoreInterprocessMemory.h"
#include "IPC/TextureShareCoreInterprocessHelpers.h"

#include "Module/TextureShareCoreModule.h"
#include "Module/TextureShareCoreLog.h"

#include "Misc/ScopeLock.h"

using namespace UE::TextureShareCore;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreObject
//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreObject::BeginSyncBarrier(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject, const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass)
{
	UE_TS_BARRIER_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:BeginSyncBarrier(%s[%s]) %s In << %s >>"), *GetName(), GetTEXT(InSyncStep), GetTEXT(InSyncPass), *ToString(LocalObject), *ToString(GetFrameConnections()));

	if (LocalObject.Sync.BeginSyncBarrier(InSyncStep, InSyncPass, FindNextSyncStep(InSyncStep)))
	{
		return true;
	}

	return false;
}

bool FTextureShareCoreObject::AcceptSyncBarrier(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject, const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass)
{
	UE_TS_BARRIER_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:AcceptSyncBarrier(%s[%s]) %s In << %s >>"), *GetName(), GetTEXT(InSyncStep), GetTEXT(InSyncPass), *ToString(LocalObject), *ToString(GetFrameConnections()));

	if (LocalObject.Sync.AcceptSyncBarrier(InSyncStep, InSyncPass))
	{
		return true;
	}

	// debug purpose
	return false;
}

bool FTextureShareCoreObject::PrepareSyncBarrierPass(const ETextureShareSyncStep InSyncStep)
{
	bool bResult = false;

	// Return objects who support sync pass at this frame
	if (IsFrameSyncActive() && !IsEmptyFrameConnections() && Owner.LockInterprocessMemory(GetTimeOutSettings().MemoryMutexTimeout))
	{
		if (FTextureShareCoreInterprocessMemory* InterprocessMemory = Owner.GetInterprocessMemory())
		{
			UpdateFrameConnections(*InterprocessMemory);

			bResult = InterprocessMemory->IsUsedSyncBarrierPass(GetFrameConnections(), InSyncStep);
		}

		Owner.UnlockInterprocessMemory();
	}

	return bResult;
}

bool FTextureShareCoreObject::PrepareSyncBarrierPass_RenderThread(const ETextureShareSyncStep InSyncStep)
{
	bool bResult = false;

	// Return objects who support sync pass at this frame
	if (IsFrameSyncActive_RenderThread() && !IsEmptyFrameConnections() && Owner.LockInterprocessMemory(GetTimeOutSettings().MemoryMutexTimeout))
	{
		if (FTextureShareCoreInterprocessMemory* InterprocessMemory = Owner.GetInterprocessMemory())
		{
			UpdateFrameConnections(*InterprocessMemory);

			bResult = InterprocessMemory->IsUsedSyncBarrierPass(GetFrameConnections(), InSyncStep);
		}

		Owner.UnlockInterprocessMemory();
	}

	return bResult;
}

bool FTextureShareCoreObject::TryEnterSyncBarrier(const ETextureShareSyncStep InSyncStep) const
{
	UE_TS_BARRIER_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:TryEnterSyncBarrier(%s)"), *GetName(), GetTEXT(InSyncStep));

	switch (FrameSyncState)
	{
	case ETextureShareCoreInterprocessObjectFrameSyncState::FrameBegin:
		if ((InSyncStep > ETextureShareSyncStep::FrameBegin) && (InSyncStep < ETextureShareSyncStep::FrameEnd))
		{
			if (GetSyncSettings().FrameSyncSettings.Steps.Contains(InSyncStep))
			{
				return true;
			}

			UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s: Unsupported Frame sync pass '%s' - Ignored - sync logic broken"), *GetName(), GetTEXT(InSyncStep));

			return false;
		}
		break;

	case ETextureShareCoreInterprocessObjectFrameSyncState::FrameProxyBegin:
		if ((InSyncStep > ETextureShareSyncStep::FrameProxyBegin) && (InSyncStep < ETextureShareSyncStep::FrameProxyEnd))
		{
			if (GetSyncSettings().FrameSyncSettings.Steps.Contains(InSyncStep))
			{
				return true;
			}

			UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s: Unsupported FrameProxy sync pass '%s' - Ignored - sync logic broken"), *GetName(), GetTEXT(InSyncStep));

			return false;
		}
		break;

	default:
		break;
	}

	UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:TryEnterSyncBarrier(%s) - Skip sync step - %s"), *GetName(), GetTEXT(InSyncStep), GetTEXT(FrameSyncState));

	return false;
}

bool FTextureShareCoreObject::TryFrameProcessesBarrier(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject, const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass)
{
	// Mark last acces time for current process (this value used by other processes to detect die)
	LocalObject.Sync.UpdateLastAccessTime();

	if (IsEmptyFrameConnections())
	{
		UE_TS_BARRIER_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:TryFrameProcessesBarrier(%s) Failed"), *GetName(), *ToString(LocalObject));

		return false;
	}

#if TEXTURESHARECORE_BARRIER_DEBUGLOG
	UpdateFrameConnections(InterprocessMemory);
	UE_TS_BARRIER_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:TryFrameProcessesBarrier(%s) %s"), *GetName(), *ToString(LocalObject), *ToString(GetFrameConnections()));
#endif


	const ETextureShareInterprocessObjectSyncBarrierState BarrierState = InterprocessMemory.GetBarrierState(LocalObject, GetFrameConnections());
	switch (BarrierState)
	{
	case ETextureShareInterprocessObjectSyncBarrierState::Accept:

		if (LocalObject.Sync.IsBarrierCompleted(InSyncStep, InSyncPass))
		{
			// break wait

			if (InSyncPass == ETextureShareSyncPass::Exit)
			{
				AcceptSyncBarrier(InterprocessMemory, LocalObject, InSyncStep, ETextureShareSyncPass::Complete);
			}

			return false;
		}

		// Accept barrier
		AcceptSyncBarrier(InterprocessMemory, LocalObject, InSyncStep, InSyncPass);

		// continue loop until barrier completed
		return true;

	case ETextureShareInterprocessObjectSyncBarrierState::Wait:
		// Continue wait
		return true;

	case ETextureShareInterprocessObjectSyncBarrierState::ObjectLost:
	case ETextureShareInterprocessObjectSyncBarrierState::FrameSyncLost:
	default:
		break;
	}

	HandleFrameLost(InterprocessMemory, LocalObject);

	UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:TryFrameProcessesBarrier(%s) BarrierState Failed"), *GetName(), *ToString(LocalObject));

	// break wait
	return false;
}

bool FTextureShareCoreObject::SyncBarrierPass(const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass)
{
	UE_TS_BARRIER_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:SyncBarrierPass(%s, %s)"), *GetName(), GetTEXT(InSyncStep), GetTEXT(InSyncPass));

	bool bBarrierResult = false;
	if (IsFrameSyncActive() && Owner.LockInterprocessMemory(GetTimeOutSettings().MemoryMutexTimeout))
	{
		if (FTextureShareCoreInterprocessMemory* InterprocessMemory = Owner.GetInterprocessMemory())
		{
			if (FTextureShareCoreInterprocessObject* LocalObject = InterprocessMemory->FindObject(GetObjectDesc()))
			{
				bBarrierResult = SyncBarrierPassImpl(*InterprocessMemory, *LocalObject, InSyncStep, InSyncPass);
			};
		}

		// Wake up remote processes anywait, because we change mem object header
		SendNotificationEvents(false);

		Owner.UnlockInterprocessMemory();
	}

	if (bBarrierResult)
	{
		return true;
	}

	UE_TS_BARRIER_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:SyncBarrierPass()  FAILED"), *GetName());

	return false;
}

bool FTextureShareCoreObject::SyncBarrierPass_RenderThread(const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass)
{
	UE_TS_BARRIER_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:SyncBarrierPass_RenderThread(%s, %s)"), *GetName(), GetTEXT(InSyncStep), GetTEXT(InSyncPass));

	bool bBarrierResult = false;
	if (IsFrameSyncActive_RenderThread() && Owner.LockInterprocessMemory(GetTimeOutSettings().MemoryMutexTimeout))
	{
		if (FTextureShareCoreInterprocessMemory* InterprocessMemory = Owner.GetInterprocessMemory())
		{
			if (FTextureShareCoreInterprocessObject* LocalObject = InterprocessMemory->FindObject(GetObjectDesc()))
			{
				bBarrierResult = SyncBarrierPassImpl(*InterprocessMemory, *LocalObject, InSyncStep, InSyncPass);
			};
		}

		// Wake up remote processes anywait, because we change mem object header
		SendNotificationEvents(false);

		Owner.UnlockInterprocessMemory();
	}

	if (bBarrierResult)
	{
		return true;
	}

	UE_TS_BARRIER_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:SyncBarrierPass_RenderThread()  FAILED"), *GetName());

	return false;
}

bool FTextureShareCoreObject::SyncBarrierPassImpl(FTextureShareCoreInterprocessMemory& InterprocessMemory, FTextureShareCoreInterprocessObject& LocalObject, const ETextureShareSyncStep InSyncStep, const ETextureShareSyncPass InSyncPass)
{
	// Begin barrier
	BeginSyncBarrier(InterprocessMemory, LocalObject, InSyncStep, InSyncPass);

	// Begin processs barrier sync
	FTextureShareCoreObjectTimeout FrameSyncTimer(GetTimeOutSettings().FrameSyncTimeOut, GetTimeOutSettings().FrameSyncTimeOutSplit);

	// repeat that barrier, until the connected processes is defined
	while (TryFrameProcessesBarrier(InterprocessMemory, LocalObject, InSyncStep, InSyncPass))
	{
		// Handle Event error or timeout
		if (FrameSyncTimer.IsTimeOut())
		{
			HandleFrameLost(InterprocessMemory, LocalObject);

			return false;
		}

		// Wait for remote process data changes
		if (!TryWaitFrameProcesses(FrameSyncTimer.GetRemainMaxMillisecondsToWait()))
		{
			HandleFrameLost(InterprocessMemory, LocalObject);

			return false;
		}
	}

	return LocalObject.IsEnabled() && !IsEmptyFrameConnections();
}

ETextureShareSyncStep FTextureShareCoreObject::FindNextSyncStep(const ETextureShareSyncStep InSyncStep) const
{
	const TArray<ETextureShareSyncStep> SyncSteps = GetSyncSettings().FrameSyncSettings.Steps;
	if (SyncSteps.IsEmpty() == false)
	{
		int32 Index = (InSyncStep == ETextureShareSyncStep::InterprocessConnection) ? 0 : SyncSteps.Find(InSyncStep);
		if (Index != INDEX_NONE)
		{
			while (++Index < SyncSteps.Num())
			{
				const ETextureShareSyncStep NextStep = SyncSteps[Index];
				switch (NextStep)
				{
				case ETextureShareSyncStep::FrameBegin:
				case ETextureShareSyncStep::FrameEnd:
				case ETextureShareSyncStep::FrameProxyBegin:
				case ETextureShareSyncStep::FrameProxyEnd:
				case ETextureShareSyncStep::Undefined:
					// Skip internal values
					break;

				default:
					if (IsEmptyFrameConnections())
					{
						return NextStep;
					}

					// Use next step only if any process support
					for (const FTextureShareCoreObjectDesc& ObjIt : GetFrameConnections())
					{
						if (ObjIt.Sync.IsStepEnabled(NextStep))
						{
							return NextStep;
						}
					}
				}
			}

			// new frame connection
			return ETextureShareSyncStep::InterprocessConnection;
		}
	}

	return ETextureShareSyncStep::Undefined;
}
