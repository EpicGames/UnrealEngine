// Copyright Epic Games, Inc. All Rights Reserved.

#include "Object/TextureShareCoreObject.h"
#include "Object/TextureShareCoreObjectContainers.h"

#include "IPC/TextureShareCoreInterprocessMemoryRegion.h"
#include "IPC/TextureShareCoreInterprocessMutex.h"
#include "IPC/TextureShareCoreInterprocessEvent.h"
#include "IPC/Containers/TextureShareCoreInterprocessMemory.h"
#include "IPC/TextureShareCoreInterprocessHelpers.h"

#include "Module/TextureShareCoreModule.h"
#include "Module/TextureShareCoreLog.h"

#include "Misc/ScopeLock.h"
#include "HAL/IConsoleManager.h"

using namespace UE::TextureShareCore;

/** Use lock for internal data that can be used by multiple threads at the same time. */
int32 GTextureShareMultithreadDataLock = 1;
static FAutoConsoleVariableRef CVarTextureShareMultithreadDataLock(
	TEXT("TextureShare.MultithreadDataLock"),
	GTextureShareMultithreadDataLock,
	TEXT("Lock multithread data by the mutex. (Default = 1)"),
	ECVF_Default
);

/** Helper functions. */
namespace UE::TextureShareCore::Helpers
{
	/** true if this thread mutex enabled. */
	static inline bool IsThreadMutexEnabled(const ETextureShareThreadMutex InThreadMutex)
	{
		switch (InThreadMutex)
		{
		case ETextureShareThreadMutex::InternalObjectDescMT:
		case ETextureShareThreadMutex::InternalFrameConnectionsMT:
		case ETextureShareThreadMutex::InternalSyncSettingsMT:
			return GTextureShareMultithreadDataLock != 0;
		}

		return true;
	}
};

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareCoreObject
//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareCoreObject::LockThreadMutex(const ETextureShareThreadMutex InThreadMutex, bool bForceLockNoWait) const
{
	if (!UE::TextureShareCore::Helpers::IsThreadMutexEnabled(InThreadMutex))
	{
		return true;
	}

	TSharedPtr<FTextureShareCoreInterprocessMutex, ESPMode::ThreadSafe> ThreadMutex = GetThreadMutex(InThreadMutex);
	if (!ThreadMutex.IsValid())
	{
		UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:LockThreadMutex(%s) Mutex object not exist"), *GetName(), GetTEXT(InThreadMutex));

		return true;
	}

	if (bForceLockNoWait)
	{
		UE_TS_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:LockThreadMutex(%s) ForceLock"), *GetName(), GetTEXT(InThreadMutex));

		return ThreadMutex->LockMutex(0);
	}

	if (InThreadMutex < ETextureShareThreadMutex::MAX_LOG)
	{
		UE_TS_LOG(LogTextureShareCoreObjectSync, VeryVerbose, TEXT("%s:LockThreadMutex(%s) try"), *GetName(), GetTEXT(InThreadMutex));
	}

	if (ThreadMutex->LockMutex(GetTimeOutSettings().ThreadMutexTimeout))
	{
		if (InThreadMutex < ETextureShareThreadMutex::MAX_LOG)
		{
			UE_TS_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:LockThreadMutex(%s)"), *GetName(), GetTEXT(InThreadMutex));
		}

		return true;
	}

	// mutex deadlock
	UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:LockThreadMutex(%s) DEADLOCK"), *GetName(), GetTEXT(InThreadMutex));

	// Unlock stale mutex
	ThreadMutex->UnlockMutex();

	return false;
}

bool FTextureShareCoreObject::UnlockThreadMutex(const ETextureShareThreadMutex InThreadMutex) const
{
	if (!UE::TextureShareCore::Helpers::IsThreadMutexEnabled(InThreadMutex))
	{
		return true;
	}

	TSharedPtr<FTextureShareCoreInterprocessMutex, ESPMode::ThreadSafe> ThreadMutex = GetThreadMutex(InThreadMutex);
	if (!ThreadMutex.IsValid())
	{
		UE_TS_LOG(LogTextureShareCoreObjectSync, Error, TEXT("%s:UnlockThreadMutex(%s) Mutex object not exist"), *GetName(), GetTEXT(InThreadMutex));

		return true;
	}

	if (InThreadMutex < ETextureShareThreadMutex::MAX_LOG)
	{
		UE_TS_LOG(LogTextureShareCoreObjectSync, Log, TEXT("%s:UnlockThreadMutex(%s)"), *GetName(), GetTEXT(InThreadMutex));
	}

	ThreadMutex->UnlockMutex();

	return true;
}

//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareCoreObject::InitializeThreadMutexes()
{
	for (int32 Index = 0; Index < (uint8)ETextureShareThreadMutex::COUNT; Index++)
	{
		ThreadMutexes.Add(MakeShared<FTextureShareCoreInterprocessMutex, ESPMode::ThreadSafe>());
		if (ThreadMutexes.Last().IsValid())
		{
			ThreadMutexes.Last()->Initialize();
		}
	}
}

TSharedPtr<FTextureShareCoreInterprocessMutex, ESPMode::ThreadSafe> FTextureShareCoreObject::GetThreadMutex(const ETextureShareThreadMutex InThreadMutex) const
{
	const uint8 ThreadMutexIndex = (uint8)InThreadMutex;
	if (ThreadMutexes.IsValidIndex(ThreadMutexIndex))
	{
		if (ThreadMutexes[ThreadMutexIndex].IsValid() && ThreadMutexes[ThreadMutexIndex]->IsValid())
		{
			return ThreadMutexes[ThreadMutexIndex];
		}
	}

	return nullptr;
}

void FTextureShareCoreObject::ReleaseThreadMutexes()
{
	ThreadMutexes.Empty();
}
