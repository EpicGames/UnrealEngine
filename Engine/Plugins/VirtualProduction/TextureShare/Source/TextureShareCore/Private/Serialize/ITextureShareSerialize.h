// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if __UNREAL__
#include "CoreMinimal.h"
#endif

#include "Containers/TextureShareCoreEnums.h"

#if __UNREAL__
#include "PixelFormat.h"
#else
// Provides an enumeration of UE pixel formats for the SDK.
#include "Containers/UnrealEngine/TextureShareSDKUnrealEngineEnums_PixelFormat.h"
#endif

#include <DXGIFormat.h>

class ITextureShareSerializeStream;

/**
 * Serialize structures data to exchange between processes
 */
struct ITextureShareSerialize
{
	virtual ~ITextureShareSerialize() = default;
	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream& Stream) = 0;
};

/**
 * Interface for binary serialization
 */
class ITextureShareSerializeStream
{
public:
	virtual ~ITextureShareSerializeStream() = default;

public:
	// Serialize any serializable objects
	virtual ITextureShareSerializeStream& operator<<(ITextureShareSerialize& InOutData) = 0;

public:
	// Serialize atomic types:
	virtual ITextureShareSerializeStream& operator<<(int8& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(int16& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(int32& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(int64& In) = 0;

	virtual ITextureShareSerializeStream& operator<<(uint8& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(uint16& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(uint32& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(uint64& In) = 0;

	virtual ITextureShareSerializeStream& operator<<(float& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(double& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(bool& In) = 0;

public:
	// Serialize TextureShare enums:
	virtual ITextureShareSerializeStream& operator<<(ETextureShareFrameSyncTemplate& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(ETextureShareSyncPass& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(ETextureShareSyncState& In) = 0;

	virtual ITextureShareSerializeStream& operator<<(ETextureShareSyncStep& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(ETextureShareResourceType& In) = 0;
	
	virtual ITextureShareSerializeStream& operator<<(ETextureShareTextureOp& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(ETextureShareEyeType& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(ETextureShareProcessType& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(ETextureShareDeviceType& In) = 0;

	virtual ITextureShareSerializeStream& operator<<(ETextureShareCoreSceneViewManualProjectionType& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(ETextureShareViewRotationDataType& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(ETextureShareViewLocationDataType& In) = 0;

public:
	// Serialize other enums
	virtual ITextureShareSerializeStream& operator<<(EPixelFormat& In) = 0;
	virtual ITextureShareSerializeStream& operator<<(DXGI_FORMAT& In) = 0;

public:
	// Serialize UECore types:
	virtual ITextureShareSerializeStream& operator<<(FString& InOutData) = 0;
	virtual ITextureShareSerializeStream& operator<<(FVector& InOutData) = 0;
	virtual ITextureShareSerializeStream& operator<<(FVector2D& InOutData) = 0;
	virtual ITextureShareSerializeStream& operator<<(FRotator& InOutData) = 0;
	virtual ITextureShareSerializeStream& operator<<(FQuat& InOutData) = 0;
	virtual ITextureShareSerializeStream& operator<<(FMatrix& InOutData) = 0;
	virtual ITextureShareSerializeStream& operator<<(FIntRect& InOutData) = 0;
	virtual ITextureShareSerializeStream& operator<<(FIntPoint& InOutData) = 0;
	virtual ITextureShareSerializeStream& operator<<(FGuid& InOutData) = 0;
	virtual ITextureShareSerializeStream& operator<<(Windows::HANDLE& InOutData) = 0;

public:
	virtual bool IsWriteStream() const = 0;
	virtual ITextureShareSerializeStream& SerializeData(void*, const uint32_t)
	{
		return *this;
	}

	virtual ITextureShareSerializeStream& SerializeData(const void*, const uint32_t)
	{
		return *this;
	}
};
