// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Serialize/ITextureShareSerialize.h"

/**
 * TTextureShareCoreRemoveReference<type> will remove any references from a type.
 */
template <typename T> struct TTextureShareCoreRemoveReference { typedef T Type; };
template <typename T> struct TTextureShareCoreRemoveReference<T& > { typedef T Type; };
template <typename T> struct TTextureShareCoreRemoveReference<T&&> { typedef T Type; };

/**
 * Forward will cast a reference to an rvalue reference.
 * This is UE's equivalent of std::forward.
 */
template <typename T>
inline T&& TextureShareCoreForward(typename TTextureShareCoreRemoveReference<T>::Type& Obj)
{
	return (T&&)Obj;
}

template <typename T>
inline T&& TextureShareCoreForward(typename TTextureShareCoreRemoveReference<T>::Type&& Obj)
{
	return (T&&)Obj;
}

/**
 * Invokes a callable with a set of arguments.  Allows the following:
 * Copied from file "Templates\Invoke.h" for SDK
 */
template <typename FuncType, typename... ArgTypes>
inline auto TextureShareCoreInvoke(FuncType&& Func, ArgTypes&&... Args)
	-> decltype(TextureShareCoreForward<FuncType>(Func)(TextureShareCoreForward<ArgTypes>(Args)...))
{
	return TextureShareCoreForward<FuncType>(Func)(TextureShareCoreForward<ArgTypes>(Args)...);
}

/**
 * Serializable array template for TextureShare
 */
template<typename T>
class TArraySerializable
	: public ITextureShareSerialize
	, public TArray<T>
{
public:
	virtual ITextureShareSerializeStream& Serialize(ITextureShareSerializeStream& Stream) override
	{
		uint32 ItemsCount = this->Num();
		Stream << ItemsCount;

		if(!Stream.IsWriteStream())
		{
			this->AddDefaulted(ItemsCount);
		}

		for (T& It : *this)
		{
			Stream << It;
		}

		return Stream;
	}

public:
	// Append array values ordered. Value must support compare ops
	void AppendSorted(const TArray<T>& InArrayValues)
	{
		for (const T& ValueIt : InArrayValues)
		{
			AddSorted(ValueIt);
		}
	}

	// Add value ordered. Value must support compare ops
	void AddSorted(const T InValue)
	{
		if (this->Find(InValue) == INDEX_NONE)
		{
			for (int32 Index = this->Num() - 1; Index >= 0; Index--)
			{
				if (this->operator[](Index) < InValue)
				{
					this->Insert(InValue, Index + 1);
					break;
				}
			}
		}
	}

	// Returns true if two array values are equal
	bool EqualsFunc(const TArray<T>& InArrayValues) const
	{
		if (this->Num() == InArrayValues.Num())
		{
			for (int32 Index = 0; Index < this->Num(); Index++)
			{
				if (!(this->operator[](Index) == InArrayValues[Index]))
				{
					return false;
				}
			}

			return true;
		}

		return false;
	}

	/**
	 * Finds an element which matches a predicate functor.
	 *
	 * @param Pred The functor to apply to each element. true, or nullptr if none is found.
	 * @see FilterByPredicate, ContainsByPredicate
	 */
	template <typename Predicate>
	T* FindByPredicate(Predicate Pred)
	{
		for (int32 Index = 0; Index < this->Num(); Index++)
		{
			T& ValueIt = this->operator[](Index);
			if (TextureShareCoreInvoke(Pred, ValueIt))
			{
				return &ValueIt;
			}
		}

		return nullptr;
	}

	template <typename Predicate>
	const T* FindByPredicate(Predicate Pred) const
	{
		for (int32 Index = 0; Index < this->Num(); Index++)
		{
			const T& ValueIt = this->operator[](Index);
			if (TextureShareCoreInvoke(Pred, ValueIt))
			{
				return &ValueIt;
			}
		}

		return nullptr;
	}

	/**
	 * Finds an item by predicate.
	 *
	 * @param Pred The predicate to match.
	 * @returns Index to the first matching element, or INDEX_NONE if none is found.
	 */
	template <typename Predicate>
	int32 IndexOfByPredicate(Predicate Pred) const
	{
		for (int32 Index = 0; Index < this->Num(); Index++)
		{
			const T& ValueIt = this->operator[](Index);
			if (TextureShareCoreInvoke(Pred, ValueIt))
			{
				return Index;
			}
		}

		return INDEX_NONE;
	}

	template<typename SubT>
	const T* FindByEqualsFunc(const SubT& InValue) const
	{
		return FindByPredicate([InValue](const T& DataIt) {
			return DataIt.EqualsFunc(InValue);
		});
	}

	template<typename SubT>
	T* FindByEqualsFunc(const SubT& InValue)
	{
		return FindByPredicate([InValue](const T& DataIt) {
			return DataIt.EqualsFunc(InValue);
		});
	}

	template<typename SubT>
	bool GetValuesByEqualsFunc(const SubT& InValue, TArray<T>& OutValues) const
	{
		for (int32 Index = 0; Index < this->Num(); Index++)
		{
			const T& ValueIt = this->operator[](Index);
			if (ValueIt.EqualsFunc(InValue))
			{
				OutValues.Add(ValueIt);
			}
		}

		return OutValues.Num() > 0;
	}

	bool EmplaceValue(const T& InValue, const bool bOverrideExist = false)
	{
		if (T* ExistValue = FindByPredicate([InValue](const T& DataIt) {
			return DataIt.EqualsFunc(InValue);
		}))
		{
			if (bOverrideExist)
			{
				*ExistValue = InValue;
			}

			return bOverrideExist;
		}

		this->Add(InValue);

		return true;
	}

	bool AppendValues(const TArraySerializable<T>& In, const bool bOverrideExist = false)
	{
		bool bResult = true;

		for (const T& ResourceIt : *this)
		{
			if (EmplaceValue(ResourceIt, bOverrideExist) == false)
			{
				bResult = false;
			}
		}

		return bResult;
	}
};
