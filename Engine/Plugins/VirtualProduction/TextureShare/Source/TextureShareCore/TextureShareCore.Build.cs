// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildTool;

[SupportedPlatforms("Win64")]
public class TextureShareCore : ModuleRules
{
	public TextureShareCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
		});

		// Show more log for internal sync processes
		bool bEnableExtraDebugLog = false;
		if (bEnableExtraDebugLog)
		{
			PublicDefinitions.Add("TEXTURESHARECORE_DEBUGLOG=1");
		}
		else
		{
			PublicDefinitions.Add("TEXTURESHARECORE_DEBUGLOG=0");
		}

		bool bEnableExtraDebugLogForInternalBarriers = false;
		if (bEnableExtraDebugLogForInternalBarriers && bEnableExtraDebugLog)
		{
			PublicDefinitions.Add("TEXTURESHARECORE_BARRIER_DEBUGLOG=1");
		}
		else
		{
			PublicDefinitions.Add("TEXTURESHARECORE_BARRIER_DEBUGLOG=0");
		}

		// Allow using Vulkan render device
		bool bSupportDeviceVulkan = false;

		if (bSupportDeviceVulkan)
		{
			PublicDefinitions.Add("TEXTURESHARECORE_VULKAN=1");

			AddEngineThirdPartyPrivateStaticDependencies(Target, "Vulkan");

			if (Target.bShouldCompileAsDLL==false)
			{
				PublicDefinitions.Add("TEXTURESHARECORE_INITIALIZEVULKANEXTENSIONS");

				PrivateDependencyModuleNames.AddRange(new string[] { "VulkanRHI" });
				PrivateIncludePathModuleNames.Add("VulkanRHI");
			}
		}
		else
		{
			PublicDefinitions.Add("TEXTURESHARECORE_VULKAN=0");
		}

		if (Target.bShouldCompileAsDLL)
		{
			// Special build rules for SDK
			PublicDefinitions.Add("TEXTURESHARECORE_SDK=1");
		}
		else
		{
			PublicDefinitions.Add("TEXTURESHARECORE_SDK=0");
		}

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			PublicIncludePathModuleNames.AddAll("D3D11RHI", "D3D12RHI");

			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX12");
		}
	}
}
