// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

/**
 * Private API for nDisplay integration with TextureShare
 */
class TEXTURESHAREDISPLAYCLUSTER_API ITextureShareDisplayClusterAPI
{
public:
	virtual ~ITextureShareDisplayClusterAPI() = default;

public:
	/**
	 * Set manual projection data 
	 *
	 * @param InPolicy   - projection policy
	 * @param InProjectionData - manual prj data
	 *
	 * @return true, if success
	 */
	virtual bool TextureSharePolicySetProjectionData(const TSharedPtr<class IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InPolicy, const TArray<struct FTextureShareCoreManualProjection>& InProjectionData) = 0;
};
