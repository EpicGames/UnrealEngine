// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if __UNREAL__
#include "CoreMinimal.h"
#else
#include "windows.h"
#endif

/**
 * Constant names for nDisplay texture resources
 */
namespace UE::TextureShare::DisplayClusterStrings
{
	// Default ShareName for nDisplay integration object
	static constexpr auto DefaultShareName = TEXT("nDisplay");

	namespace Viewport
	{
		// FinalColor, but can be overrided
		static constexpr auto FinalColor = TEXT("ViewportFinalColor");

		// internal resolved shader resources, used as warp&blend source
		static constexpr auto Input = TEXT("ViewportInput");
		static constexpr auto Mips = TEXT("ViewportMips");

		// After warp viewport (before output remap)
		static constexpr auto Warped = TEXT("ViewportWarpBlend");
	}

	namespace Output
	{
		// access to nDisplay frame backbuffer
		static constexpr auto Backbuffer = TEXT("FrameBackbuffer");
		static constexpr auto BackbufferTemp = TEXT("FrameBackbufferTemp");
	}

	namespace Postprocess
	{
		static constexpr auto TextureShare = TEXT("TextureShare");
	}

	namespace Projection
	{
		static constexpr auto TextureShare = TEXT("textureshare");
	}
};
