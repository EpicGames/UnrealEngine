// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/TextureShareDisplayClusterAPI.h"
#include "Module/TextureShareDisplayClusterLog.h"
#include "Misc/TextureShareDisplayClusterStrings.h"

#include "Projection/TextureShareProjectionPolicyFactory.h"
#include "Projection/TextureShareProjectionPolicy.h"
#include "PostProcess/TextureSharePostprocessFactory.h"

#include "IDisplayCluster.h"

#include "Render/Projection/IDisplayClusterProjectionPolicy.h"
#include "Render/IDisplayClusterRenderManager.h"

namespace UE::TextureShare::DisplayCluster
{
	static IDisplayCluster& DisplayClusterAPI()
	{
		static IDisplayCluster& DisplayClusterSingleton = IDisplayCluster::Get();
		return DisplayClusterSingleton;
	}
};
using namespace UE::TextureShare::DisplayCluster;
using namespace UE::TextureShare;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareDisplayClusterAPI
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareDisplayClusterAPI::FTextureShareDisplayClusterAPI()
{
	TSharedPtr<IDisplayClusterProjectionPolicyFactory> Factory;

	// TextureShare projection
	Factory = MakeShared<FTextureShareProjectionPolicyFactory>();
	ProjectionPolicyFactories.Emplace(DisplayClusterStrings::Projection::TextureShare, Factory);

	TSharedPtr<IDisplayClusterPostProcessFactory> Postprocess;

	Postprocess = MakeShared<FTextureSharePostprocessFactory>();
	PostprocessAssets.Emplace(DisplayClusterStrings::Postprocess::TextureShare, Postprocess);
}

FTextureShareDisplayClusterAPI::~FTextureShareDisplayClusterAPI()
{ }

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareDisplayClusterAPI::StartupModule()
{
	if (!IDisplayCluster::IsAvailable())
	{
		UE_LOGF(LogTextureShareDisplayCluster, Error, "nDisplay plugin not found.");

		return false;
	}

	IDisplayClusterRenderManager* RenderMgr = DisplayClusterAPI().GetRenderMgr();
	if (RenderMgr)
	{
		for (auto it = ProjectionPolicyFactories.CreateIterator(); it; ++it)
		{
			UE_LOGF(LogTextureShareDisplayCluster, Verbose, "Registering <%ls> projection policy factory...", *it->Key);

			if (!RenderMgr->RegisterProjectionPolicyFactory(it->Key, it->Value))
			{
				UE_LOGF(LogTextureShareDisplayCluster, Warning, "Couldn't register <%ls> projection policy factory", *it->Key);
			}
		}

		for (TPair<FString, TSharedPtr<IDisplayClusterPostProcessFactory>>& PPFactoryIt : PostprocessAssets)
		{
			UE_LOGF(LogTextureShareDisplayCluster, Verbose, "Registering <%ls> postprocess factory...", *PPFactoryIt.Key);

			if (!RenderMgr->RegisterPostProcessFactory(PPFactoryIt.Key, PPFactoryIt.Value))
			{
				UE_LOGF(LogTextureShareDisplayCluster, Warning, "Couldn't register <%ls> postprocess factory", *PPFactoryIt.Key);
			}
		}
	}

	bModuleActive = true;

	return true;
}

void FTextureShareDisplayClusterAPI::ShutdownModule()
{
	if (bModuleActive)
	{
		bModuleActive = false;

		IDisplayClusterRenderManager* RenderMgr = DisplayClusterAPI().GetRenderMgr();
		if (RenderMgr)
		{
			for (auto it = ProjectionPolicyFactories.CreateConstIterator(); it; ++it)
			{
				UE_LOGF(LogTextureShareDisplayCluster, Verbose, "Un-registering <%ls> projection factory...", *it->Key);

				if (!RenderMgr->UnregisterProjectionPolicyFactory(it->Key))
				{
					UE_LOGF(LogTextureShareDisplayCluster, Warning, "An error occurred during un-registering the <%ls> projection factory", *it->Key);
				}
			}

			for (TPair<FString, TSharedPtr<IDisplayClusterPostProcessFactory>>& PPFactoryIt : PostprocessAssets)
			{
				UE_LOGF(LogTextureShareDisplayCluster, Verbose, "Un-registering <%ls> postprocess factory...", *PPFactoryIt.Key);

				if (!RenderMgr->UnregisterPostProcessFactory(PPFactoryIt.Key))
				{
					UE_LOGF(LogTextureShareDisplayCluster, Warning, "An error occurred during un-registering the <%ls> postprocess  factory", *PPFactoryIt.Key);
				}
			}
		}
	}
}

bool FTextureShareDisplayClusterAPI::TextureSharePolicySetProjectionData(const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InPolicy, const TArray<FTextureShareCoreManualProjection>& InProjectionData)
{
	if (bModuleActive)
	{
		if (InPolicy.IsValid())
		{
			if (FTextureShareProjectionPolicy* TSPolicyObject = static_cast<FTextureShareProjectionPolicy*>(InPolicy.Get()))
			{
				return TSPolicyObject->SetCustomProjection(InProjectionData);
			}
		}
	}

	return false;
}
