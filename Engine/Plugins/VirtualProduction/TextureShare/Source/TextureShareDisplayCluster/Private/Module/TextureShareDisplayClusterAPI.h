// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ITextureShareDisplayClusterAPI.h"

/**
 * Private API for nDisplay integration with TextureShare implementation
 */
class FTextureShareDisplayClusterAPI
	: public ITextureShareDisplayClusterAPI
{
public:
	FTextureShareDisplayClusterAPI();
	virtual ~FTextureShareDisplayClusterAPI();

public:
	virtual bool TextureSharePolicySetProjectionData(const TSharedPtr<class IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& InPolicy, const TArray<FTextureShareCoreManualProjection>& InProjectionData) override;

public:
	bool StartupModule();
	void ShutdownModule();

private:
	// Available projection policies factories
	TMap<FString, TSharedPtr<class IDisplayClusterProjectionPolicyFactory>> ProjectionPolicyFactories;

	// Available postprocess factories
	TMap<FString, TSharedPtr<class IDisplayClusterPostProcessFactory>> PostprocessAssets;

	bool bModuleActive = false;
};
