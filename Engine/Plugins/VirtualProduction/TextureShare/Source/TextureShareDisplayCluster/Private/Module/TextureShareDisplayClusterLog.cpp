// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/TextureShareDisplayClusterLog.h"

//////////////////////////////////////////////////////////////////////////////////////////////
DEFINE_LOG_CATEGORY(LogTextureShareDisplayCluster);
DEFINE_LOG_CATEGORY(LogTextureShareDisplayClusterProjection);
DEFINE_LOG_CATEGORY(LogTextureShareDisplayClusterPostProcess);
