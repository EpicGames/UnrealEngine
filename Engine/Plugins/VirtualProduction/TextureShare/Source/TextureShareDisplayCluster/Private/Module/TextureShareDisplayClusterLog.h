// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


#if TEXTURESHAREDISPLAYCLUSTER_DEBUGLOG
// Enable extra log
#define UE_TS_LOG(CategoryName, Verbosity, Format, ...)\
	UE_LOG(CategoryName, Verbosity, Format, ##__VA_ARGS__)

#else
// Disable extra log
#define UE_TS_LOG(CategoryName, Verbosity, Format, ...)
#endif

// Plugin-wide log categories
#if UE_BUILD_SHIPPING
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareDisplayCluster, Warning, Warning);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareDisplayClusterProjection, Warning, Warning);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareDisplayClusterPostProcess, Warning, Warning);
#else
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareDisplayCluster, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareDisplayClusterProjection, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogTextureShareDisplayClusterPostProcess, Log, All);
#endif
