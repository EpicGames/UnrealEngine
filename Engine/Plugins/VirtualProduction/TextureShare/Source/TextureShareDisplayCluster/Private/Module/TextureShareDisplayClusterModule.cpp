// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/TextureShareDisplayClusterModule.h"
#include "Module/TextureShareDisplayClusterLog.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareDisplayCluster
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareDisplayCluster::FTextureShareDisplayCluster()
{
	TextureShareDisplayClusterAPI = MakeUnique<FTextureShareDisplayClusterAPI>();
}

FTextureShareDisplayCluster::~FTextureShareDisplayCluster()
{
	TextureShareDisplayClusterAPI.Reset();
	UE_LOGF(LogTextureShareDisplayCluster, Log, "TextureShareDisplayCluster module has been destroyed");
}

//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureShareDisplayCluster::StartupModule()
{
	UE_LOGF(LogTextureShareDisplayCluster, Log, "TextureShareDisplayCluster module startup");
	if (TextureShareDisplayClusterAPI->StartupModule())
	{
		UE_LOGF(LogTextureShareDisplayCluster, Log, "TextureShareDisplayCluster module has started");
	}
	else
	{
		UE_LOGF(LogTextureShareDisplayCluster, Log, "TextureShareDisplayCluster module disabled");
	}
}

void FTextureShareDisplayCluster::ShutdownModule()
{
	UE_LOGF(LogTextureShareDisplayCluster, Log, "TextureShareDisplayCluster module shutdown");
	TextureShareDisplayClusterAPI->ShutdownModule();
}

ITextureShareDisplayClusterAPI& FTextureShareDisplayCluster::GetTextureShareDisplayClusterAPI()
{
	return *TextureShareDisplayClusterAPI;
}

IMPLEMENT_MODULE(FTextureShareDisplayCluster, TextureShareDisplayCluster);
