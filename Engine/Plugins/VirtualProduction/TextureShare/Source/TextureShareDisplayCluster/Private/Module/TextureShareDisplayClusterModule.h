// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ITextureShareDisplayCluster.h"
#include "Module/TextureShareDisplayClusterAPI.h"

/**
 * Module Impl
 */
class FTextureShareDisplayCluster
	: public ITextureShareDisplayCluster
{
public:
	FTextureShareDisplayCluster();
	virtual ~FTextureShareDisplayCluster();

public:
	// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// ~IModuleInterface

	virtual ITextureShareDisplayClusterAPI& GetTextureShareDisplayClusterAPI() override;

protected:
	TUniquePtr<FTextureShareDisplayClusterAPI> TextureShareDisplayClusterAPI;
};
