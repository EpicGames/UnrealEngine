// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/TextureSharePostprocess.h"
#include "PostProcess/TextureSharePostprocessContext.h"

#include "Engine/GameViewportClient.h"
#include "Containers/TextureShareCoreEnums.h"
#include "Misc/TextureShareDisplayClusterStrings.h"
#include "Module/TextureShareDisplayClusterLog.h"

#include "ITextureShare.h"
#include "ITextureShareAPI.h"
#include "ITextureShareObject.h"
#include "ITextureShareObjectProxy.h"
#include "ITextureShareDisplayCluster.h"

#include "IDisplayCluster.h"
#include "Game/IDisplayClusterGameManager.h"
#include "DisplayClusterRootActor.h"
#include "Config/IDisplayClusterConfigManager.h"

#include "Render/Viewport/IDisplayClusterViewportManager.h"

int32 GTextureShareEnableDisplayCluster = 1;
static FAutoConsoleVariableRef CVarTextureShareEnableDisplayCluster(
	TEXT("TextureShare.Enable.nDisplay"),
	GTextureShareEnableDisplayCluster,
	TEXT("Enable nDisplay support for TextureShare (0 = disabled)\n"),
	ECVF_RenderThreadSafe
);

namespace UE::TextureShare::PostProcess
{
	static ITextureShareAPI& TextureShareAPI()
	{
		static ITextureShareAPI& TextureShareAPISingleton = ITextureShare::Get().GetTextureShareAPI();
		return TextureShareAPISingleton;
	}

	static FViewport* GetDisplayViewport(IDisplayClusterViewportManager* InViewportManager)
	{
		if (InViewportManager)
		{
			if (UWorld* CurrentWorld = InViewportManager->GetConfiguration().GetCurrentWorld())
			{
				if (UGameViewportClient* GameViewportClientPtr = CurrentWorld->GetGameViewport())
				{
					return GameViewportClientPtr->Viewport;
				}
			}
		}

		return nullptr;
	};
};
using namespace UE::TextureShare::PostProcess;
using namespace UE::TextureShare;

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureSharePostprocess
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureSharePostprocess::FTextureSharePostprocess(const FString& PostprocessId, const struct FDisplayClusterConfigurationPostprocess* InConfigurationPostprocess)
	: FTextureSharePostprocessBase(PostprocessId, InConfigurationPostprocess)
{ }

FTextureSharePostprocess::~FTextureSharePostprocess()
{ }

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureSharePostprocess::IsEnabled() const
{
	return Object.IsValid() && ObjectProxy.IsValid() && GTextureShareEnableDisplayCluster != 0;
}

const FString& FTextureSharePostprocess::GetType() const
{
	static const FString Type(DisplayClusterStrings::Postprocess::TextureShare);

	return Type;
}

void FTextureSharePostprocess::ReleaseDisplayClusterPostProcessTextureShare()
{
	ObjectProxy.Reset();
	Object.Reset();

	TextureShareAPI().RemoveObject(DisplayClusterStrings::DefaultShareName);
}

bool FTextureSharePostprocess::HandleStartScene(IDisplayClusterViewportManager* InViewportManager)
{
	if (TextureShareAPI().IsObjectExist(DisplayClusterStrings::DefaultShareName))
	{
		// The old object can still exists, when referenced.
		// In this case, we show this warning to know that the TS object is still referenced by someone.
		UE_LOGF(LogTextureShareDisplayClusterPostProcess, Warning, "TextureShareDisplayCluster: TS object for nDisplay are still referenced by someone");

		return false;
	}

	// Re-use TextureShare object for nDisplay
	{
		Object = TextureShareAPI().GetOrCreateObject(DisplayClusterStrings::DefaultShareName);
		if (Object.IsValid())
		{
			// Create and assign unique TS context:
			{
				// Create new TS context:
				TSharedRef<FTextureSharePostprocessContext, ESPMode::ThreadSafe> TextureShareContext =
					MakeShared<FTextureSharePostprocessContext, ESPMode::ThreadSafe>();

				// Sets a new TS context
				Object->SetTextureShareContext(TextureShareContext);
			}

			// Set unique process name
			FString UniqueProcessId;
			{
				IDisplayCluster& DisplayCluster = IDisplayCluster::Get();
				if (ADisplayClusterRootActor* RootActor = DisplayCluster.GetGameMgr()->GetRootActor())
				{
					const FString RootActorName = RootActor->GetName();
					const FString LocalNodeId = DisplayCluster.GetConfigMgr()->GetLocalNodeId();

					// Generate unique process name
					UniqueProcessId = FString::Printf(TEXT("%s::%s"), *RootActorName, *LocalNodeId);
				}

				if (UniqueProcessId.IsEmpty())
				{
					// Use process GUID as name
					UniqueProcessId = Object->GetObjectDesc().ProcessDesc.ProcessGuid.ToString(EGuidFormats::Digits);
				}

				UE_TS_LOG(LogTextureShareDisplayClusterPostProcess, Log, TEXT("%s:SetProcessName '%s'"), *Object->GetName(), *UniqueProcessId);
				Object->SetProcessId(UniqueProcessId);

			}

			ObjectProxy = Object->GetProxy();
		}

		if (IsEnabled())
		{
			// Initialize sync settings for nDisplay
			FTextureShareCoreSyncSettings SyncSetting;
			SyncSetting.FrameSyncSettings = Object->GetFrameSyncSettings(ETextureShareFrameSyncTemplate::DisplayCluster);

			Object->SetSyncSetting(SyncSetting);

			Object->BeginSession(GetDisplayViewport(InViewportManager));

			UE_LOGF(LogTextureShareDisplayClusterPostProcess, Log, "TextureShareDisplayCluster: Initialized");

			return true;
		}
		else
		{
			ReleaseDisplayClusterPostProcessTextureShare();

			UE_LOGF(LogTextureShareDisplayClusterPostProcess, Error, "TextureShareDisplayCluster: Failed - initization failed");
		}
	}

	return false;
}

void FTextureSharePostprocess::HandleEndScene(IDisplayClusterViewportManager* InViewportManager)
{
	if (Object.IsValid())
	{
		ReleaseDisplayClusterPostProcessTextureShare();
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureSharePostprocess::HandleSetupNewFrame(IDisplayClusterViewportManager* InViewportManager)
{
	if (IsEnabled() && Object->BeginFrameSync() && Object->IsFrameSyncActive())
	{
		// Update frame marker for current frame
		Object->GetCoreData().FrameMarker.NextFrame();

		// Share defined viewports from this DC node
		UpdateSupportedViews(InViewportManager);

		// Sync IPC data (read manual projection data)
		if (Object->FrameSync(ETextureShareSyncStep::FramePreSetupBegin) && Object->IsFrameSyncActive())
		{
			// Update TS manual projection policy on this node
			UpdateManualProjectionPolicy(InViewportManager);
		}
	}
}

void FTextureSharePostprocess::HandleBeginNewFrame(IDisplayClusterViewportManager* InViewportManager, FDisplayClusterRenderFrame& InOutRenderFrame)
{
	if (IsEnabled())
	{
		if (Object->IsFrameSyncActive())
		{
			// Register viewport mapping
			UpdateViews(InViewportManager);

			if (Object->FrameSync(ETextureShareSyncStep::FrameSetupBegin) && Object->IsFrameSyncActive())
			{
				// Immediatelly begin proxy frame
				ENQUEUE_RENDER_COMMAND(DisplayClusterPostProcessTextureShare_UpdateObjectProxy)(
				[TextureSharePostprocess = SharedThis(this)](FRHICommandListImmediate& RHICmdList)
				{
					TextureSharePostprocess->BeginFrameSync_RenderThread(RHICmdList);
				});
			}
		}

		Object->EndFrameSync();
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureSharePostprocess::BeginFrameSync_RenderThread(FRHICommandListImmediate& RHICmdList)
{
	check(IsInRenderingThread());

	if (ObjectProxy.IsValid())
	{
		ObjectProxy->BeginFrameSync_RenderThread(RHICmdList);
	}
}

void FTextureSharePostprocess::HandleRenderFrameSetup_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy)
{
	check(IsInRenderingThread());

	if (IsEnabled() && ObjectProxy->IsFrameSyncActive_RenderThread())
	{
		// Share RTT with remote process
		ShareViewport_RenderThread(RHICmdList, InViewportManagerProxy, ETextureShareSyncStep::FrameProxyPreRenderEnd, EDisplayClusterViewportResourceType::InternalRenderTargetResource, DisplayClusterStrings::Viewport::FinalColor);

		ObjectProxy->FrameSync_RenderThread(RHICmdList, ETextureShareSyncStep::FrameProxyPreRenderEnd);
	}
}

void FTextureSharePostprocess::HandleBeginUpdateFrameResources_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy)
{
	check(IsInRenderingThread());

	if (IsEnabled() && ObjectProxy->IsFrameSyncActive_RenderThread())
	{
		// Share RTT with remote process
		ShareViewport_RenderThread(RHICmdList, InViewportManagerProxy, ETextureShareSyncStep::FrameProxyRenderEnd, EDisplayClusterViewportResourceType::InputShaderResource, DisplayClusterStrings::Viewport::Input);
		ShareViewport_RenderThread(RHICmdList, InViewportManagerProxy, ETextureShareSyncStep::FrameProxyRenderEnd, EDisplayClusterViewportResourceType::MipsShaderResource, DisplayClusterStrings::Viewport::Mips);

		ObjectProxy->FrameSync_RenderThread(RHICmdList, ETextureShareSyncStep::FrameProxyRenderEnd);
	}
}

void FTextureSharePostprocess::HandleUpdateFrameResourcesAfterWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy)
{
	check(IsInRenderingThread());

	if (IsEnabled() && ObjectProxy->IsFrameSyncActive_RenderThread())
	{
		// Share RTT with remote process
		ShareViewport_RenderThread(RHICmdList, InViewportManagerProxy, ETextureShareSyncStep::FrameProxyPostWarpEnd, EDisplayClusterViewportResourceType::InputShaderResource, DisplayClusterStrings::Viewport::Warped, true);

		ObjectProxy->FrameSync_RenderThread(RHICmdList, ETextureShareSyncStep::FrameProxyPostWarpEnd);
	}
}

void FTextureSharePostprocess::HandleEndUpdateFrameResources_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy)
{
	check(IsInRenderingThread());

	if (IsEnabled())
	{
		if (ObjectProxy->IsFrameSyncActive_RenderThread())
		{
			// Share per-viewport
			ShareViewport_RenderThread(RHICmdList, InViewportManagerProxy, ETextureShareSyncStep::FrameProxyPostRenderEnd, EDisplayClusterViewportResourceType::OutputFrameTargetableResource, DisplayClusterStrings::Output::Backbuffer);
			ShareViewport_RenderThread(RHICmdList, InViewportManagerProxy, ETextureShareSyncStep::FrameProxyPostRenderEnd, EDisplayClusterViewportResourceType::AdditionalFrameTargetableResource, DisplayClusterStrings::Output::BackbufferTemp);

			//Share whole backbuffer
			ShareFrame_RenderThread(RHICmdList, InViewportManagerProxy, ETextureShareSyncStep::FrameProxyPostRenderEnd, EDisplayClusterViewportResourceType::OutputFrameTargetableResource, DisplayClusterStrings::Output::Backbuffer);
			ShareFrame_RenderThread(RHICmdList, InViewportManagerProxy, ETextureShareSyncStep::FrameProxyPostRenderEnd, EDisplayClusterViewportResourceType::AdditionalFrameTargetableResource, DisplayClusterStrings::Output::BackbufferTemp);
		}

		ObjectProxy->EndFrameSync_RenderThread(RHICmdList);
	}
}
