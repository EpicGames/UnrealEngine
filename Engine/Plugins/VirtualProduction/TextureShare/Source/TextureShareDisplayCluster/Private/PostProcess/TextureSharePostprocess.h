// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "PostProcess/TextureSharePostprocessBase.h"
#include "Render/Viewport/Containers/DisplayClusterViewport_Enums.h"
#include "Containers/TextureShareContainers.h"
#include "Templates/SharedPointer.h"

#include "RHI.h"
#include "RHICommandList.h"
#include "RHIResources.h"

class FDisplayClusterRenderFrame;

class IDisplayClusterViewportManager;
class IDisplayClusterViewportManagerProxy;

/**
 * TextureShare over nDisplay post-process interface
 */
class FTextureSharePostprocess
	: public FTextureSharePostprocessBase
	, public TSharedFromThis<FTextureSharePostprocess, ESPMode::ThreadSafe>
{
public:
	FTextureSharePostprocess(const FString& PostprocessId, const struct FDisplayClusterConfigurationPostprocess* InConfigurationPostprocess);
	virtual ~FTextureSharePostprocess();

	virtual const FString& GetType() const override;

	// Handle scene loading, viewport re-new etc
	virtual bool HandleStartScene(IDisplayClusterViewportManager* InViewportManager) override;
	virtual void HandleEndScene(IDisplayClusterViewportManager* InViewportManager) override;

	// Handle frame on game thread
	virtual void HandleSetupNewFrame(IDisplayClusterViewportManager* InViewportManager) override;
	virtual void HandleBeginNewFrame(IDisplayClusterViewportManager* InViewportManager, FDisplayClusterRenderFrame& InOutRenderFrame) override;

	// Handle frame on rendering thread
	virtual void BeginFrameSync_RenderThread(FRHICommandListImmediate& RHICmdList);
	virtual void HandleRenderFrameSetup_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy) override;
	virtual void HandleBeginUpdateFrameResources_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy) override;
	virtual void HandleUpdateFrameResourcesAfterWarpBlend_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy) override;
	virtual void HandleEndUpdateFrameResources_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy) override;

	// Also share backbuffer
	virtual bool IsPostProcessFrameAfterWarpBlendRequired() const override
	{
		return true;
	}

private:
	bool IsEnabled() const;

	void ReleaseDisplayClusterPostProcessTextureShare();

	void UpdateSupportedViews(IDisplayClusterViewportManager* InViewportManager);
	void UpdateManualProjectionPolicy(IDisplayClusterViewportManager* InViewportManager);
	void UpdateViews(IDisplayClusterViewportManager* InViewportManager);

private:
	void ShareViewport_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy, const ETextureShareSyncStep InReceiveSyncStep, const EDisplayClusterViewportResourceType InResourceType, const FString& InTextureId, bool bAfterWarpBlend = false) const;
	void    ShareFrame_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy, const ETextureShareSyncStep InReceiveSyncStep, const EDisplayClusterViewportResourceType InResourceType, const FString& TextureId) const;

private:
	TSharedPtr<class ITextureShareObject, ESPMode::ThreadSafe>      Object;
	TSharedPtr<class ITextureShareObjectProxy, ESPMode::ThreadSafe> ObjectProxy;
};
