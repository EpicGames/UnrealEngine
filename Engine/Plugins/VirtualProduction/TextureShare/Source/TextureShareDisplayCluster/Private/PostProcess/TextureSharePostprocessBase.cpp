// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/TextureSharePostprocessBase.h"
#include "Misc/TextureShareDisplayClusterStrings.h"
#include "Module/TextureShareDisplayClusterLog.h"

#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "DisplayClusterConfigurationTypes.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureSharePostprocessBase
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureSharePostprocessBase::FTextureSharePostprocessBase(const FString& InProjectionPolicyId, const FDisplayClusterConfigurationPostprocess* InConfigurationPostprocess)
	: PostprocessId(InProjectionPolicyId)
{
	Parameters.Append(InConfigurationPostprocess->Parameters);
	Order = InConfigurationPostprocess->Order;
}

FTextureSharePostprocessBase::~FTextureSharePostprocessBase()
{
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureSharePostprocessBase::IsConfigurationChanged(const struct FDisplayClusterConfigurationPostprocess* InConfigurationPostprocess) const
{
	if (InConfigurationPostprocess->Type.Compare(GetType(), ESearchCase::IgnoreCase) != 0)
	{
		return true;
	}

	if (InConfigurationPostprocess->Parameters.Num() != Parameters.Num()) {

		return true;
	}

	for (const TPair<FString, FString>& NewParamIt : InConfigurationPostprocess->Parameters) {

		const FString* CurrentValue = Parameters.Find(NewParamIt.Key);
		if (CurrentValue==nullptr || CurrentValue->Compare(NewParamIt.Value, ESearchCase::IgnoreCase) != 0) {
			return true;
		}
	}

	// Parameters not changed
	return false;
}
