// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Render/PostProcess/IDisplayClusterPostProcess.h"

/**
 * Base postprocess
 */
class FTextureSharePostprocessBase
	: public IDisplayClusterPostProcess
{
public:
	FTextureSharePostprocessBase(const FString& PostprocessId, const struct FDisplayClusterConfigurationPostprocess* InConfigurationPostprocess);
	virtual ~FTextureSharePostprocessBase() = 0;

	virtual const FString& GetId() const override
	{
		return PostprocessId;
	}

	virtual int32 GetOrder() const override
	{
		return Order;
	}

	virtual const TMap<FString, FString>& GetParameters() const override
	{
		return Parameters;
	}

	virtual bool IsConfigurationChanged(const struct FDisplayClusterConfigurationPostprocess* InConfigurationPostprocess) const override;

private:
	FString PostprocessId;
	TMap<FString, FString> Parameters;
	int32 Order = -1;
};
