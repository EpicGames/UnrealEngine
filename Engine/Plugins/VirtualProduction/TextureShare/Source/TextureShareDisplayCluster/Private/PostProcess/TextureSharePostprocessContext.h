// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "ITextureShareDisplayClusterContext.h"

/**
* Custom implementation of TextureShare context for nDisplay
*/
struct FTextureSharePostprocessContext
	: public ITextureShareDisplayClusterContext
{
	// ~Begin ITextureShareDisplayClusterContext
	virtual ~FTextureSharePostprocessContext() = default;	
	// ~~ End ITextureShareDisplayClusterContext
};
