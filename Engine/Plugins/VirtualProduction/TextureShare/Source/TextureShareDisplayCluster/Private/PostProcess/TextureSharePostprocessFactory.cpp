// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/TextureSharePostprocessFactory.h"
#include "PostProcess/TextureSharePostprocess.h"

#include "Module/TextureShareDisplayClusterLog.h"
#include "DisplayClusterConfigurationTypes_Base.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureSharePostprocessFactory
//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe> FTextureSharePostprocessFactory::Create(const FString& InPostProcessId, const FDisplayClusterConfigurationPostprocess* InConfigurationPostProcess)
{
	check(InConfigurationPostProcess != nullptr);

	if (ExistsTextureSharePostprocess.IsValid())
	{
		UE_LOGF(LogTextureShareDisplayClusterPostProcess, Error, "Can't Instantiating postprocess <%ls> id='%ls' : There can only be one TextureShare postprocess per application. ", *InConfigurationPostProcess->Type, *InPostProcessId);

		return nullptr;
	}

	UE_LOGF(LogTextureShareDisplayClusterPostProcess, Verbose, "Instantiating postprocess <%ls> id='%ls'", *InConfigurationPostProcess->Type, *InPostProcessId);
	TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe> Result = MakeShared<FTextureSharePostprocess, ESPMode::ThreadSafe>(InPostProcessId, InConfigurationPostProcess);

	// Save weak ref to TS postprocess instance
	ExistsTextureSharePostprocess = Result;

	return Result;
}

bool FTextureSharePostprocessFactory::CanBeCreated(const FDisplayClusterConfigurationPostprocess* InConfigurationPostProcess) const
{
	// This postprocess used as a singletone
	return !ExistsTextureSharePostprocess.IsValid();
}
