// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/PostProcess/IDisplayClusterPostProcessFactory.h"

/**
 * Postprocess factory for TextureShare over nDisplay
 */
class FTextureSharePostprocessFactory
	: public IDisplayClusterPostProcessFactory
{
public:
	// Constructor for postprocess
	virtual TSharedPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe> Create(const FString& PostProcessId, const struct FDisplayClusterConfigurationPostprocess* InConfigurationPostProcess) override;

	virtual bool CanBeCreated(const FDisplayClusterConfigurationPostprocess* InConfigurationPostProcess) const;

private:
	// There can only be one TextureShare postprocess per application
	TWeakPtr<IDisplayClusterPostProcess, ESPMode::ThreadSafe> ExistsTextureSharePostprocess;
};
