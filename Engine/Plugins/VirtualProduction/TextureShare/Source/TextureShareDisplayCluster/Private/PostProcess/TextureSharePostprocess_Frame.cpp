// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/TextureSharePostprocess.h"

#include "Module/TextureShareDisplayClusterLog.h"
#include "Misc/TextureShareDisplayClusterStrings.h"
#include "Core/TextureShareCoreHelpers.h"

#include "Containers/TextureShareCoreEnums.h"

#include "ITextureShare.h"
#include "ITextureShareObject.h"
#include "ITextureShareDisplayCluster.h"
#include "ITextureShareDisplayClusterAPI.h"

#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureSharePostprocess
//////////////////////////////////////////////////////////////////////////////////////////////
void FTextureSharePostprocess::UpdateSupportedViews(IDisplayClusterViewportManager* InViewportManager)
{
	if (InViewportManager)
	{
		// Get all existing viewports on DC node
		for (const TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : InViewportManager->GetCurrentRenderFrameViewports())
		{
			if (ViewportIt.IsValid())
			{
				const TArray<FDisplayClusterViewport_Context>& Contexts = ViewportIt->GetContexts();
				for (const FDisplayClusterViewport_Context& ViewportContextIt : Contexts)
				{
					const ETextureShareEyeType EyeType = (Contexts.Num() == 1) ? ETextureShareEyeType::Default : (ViewportContextIt.ContextNum == 0 ? ETextureShareEyeType::StereoLeft : ETextureShareEyeType::StereoRight);
					Object->GetCoreData().SupportedViews.Add(FTextureShareCoreViewDesc(ViewportIt->GetId(), EyeType));
				}
			}
		}
	}
}

void FTextureSharePostprocess::UpdateManualProjectionPolicy(IDisplayClusterViewportManager* InViewportManager)
{
	if (InViewportManager)
	{
		for (const TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : InViewportManager->GetCurrentRenderFrameViewports())
		{
			if (ViewportIt.IsValid())
			{
				TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> PrjPolicy = ViewportIt->GetProjectionPolicy();
				if (PrjPolicy.IsValid() && PrjPolicy->GetType() == UE::TextureShare::DisplayClusterStrings::Projection::TextureShare)
				{
					// Search the custom manual projection for viewport with the "TextureShare" projection policy type 
					TArray<FTextureShareCoreManualProjection> ManualProjections;
					for (const FTextureShareCoreObjectData& FrameDataIt : Object->GetReceivedCoreObjectData())
					{
						if (FrameDataIt.Data.ManualProjections.GetValuesByEqualsFunc(ViewportIt->GetId(), ManualProjections))
						{
							// Apply manual projection
							static ITextureShareDisplayClusterAPI& TextureShareDisplayClusterAPI = ITextureShareDisplayCluster::Get().GetTextureShareDisplayClusterAPI();
							TextureShareDisplayClusterAPI.TextureSharePolicySetProjectionData(PrjPolicy, ManualProjections);

							// Save used manual projections sources
							for (const FTextureShareCoreManualProjection& It : ManualProjections)
							{
								Object->GetCoreData().ManualProjectionsSources.Add(FTextureShareCoreManualProjectionSource(It.ViewDesc, FTextureShareCoreObjectFrameMarker(FrameDataIt.Desc, FrameDataIt.Data.FrameMarker)));
							}
							break;
						}
					}
				}
			}
		}
	}
}

void FTextureSharePostprocess::UpdateViews(IDisplayClusterViewportManager* InViewportManager)
{
	if (InViewportManager)
	{
		//Register all viewport for scene eye data capture by default
		for (const TSharedPtr<IDisplayClusterViewport, ESPMode::ThreadSafe>& ViewportIt : InViewportManager->GetCurrentRenderFrameViewports())
		{
			if (ViewportIt.IsValid())
			{
				const bool bMonoscopic = ViewportIt->GetContexts().Num() == 1;
				for (const FDisplayClusterViewport_Context& ContextIt : ViewportIt->GetContexts())
				{
					const ETextureShareEyeType EyeType = bMonoscopic
						? ETextureShareEyeType::Default
						: ((ContextIt.ContextNum == 0) ? ETextureShareEyeType::StereoLeft : ETextureShareEyeType::StereoRight);

					const FTextureShareCoreViewDesc InViewDesc(ViewportIt->GetId(), EyeType);

					// Register all nDisplay viewports
					Object->GetData().Views.Add(InViewDesc, ContextIt.StereoViewIndex, ContextIt.StereoscopicPass);
				}
			}
		}
	}
}
