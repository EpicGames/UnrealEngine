// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/TextureSharePostprocess.h"
#include "Module/TextureShareDisplayClusterLog.h"
#include "Misc/TextureShareDisplayClusterStrings.h"

#include "Containers/TextureShareCoreEnums.h"
#include "Core/TextureShareCoreHelpers.h"

#include "ITextureShareObjectProxy.h"

#include "Render/Viewport/IDisplayClusterViewportManagerProxy.h"
#include "Render/Viewport/IDisplayClusterViewportProxy.h"
#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

namespace UE::TextureShare::PostProcess_FrameProxy
{
	// Support warp blend logic
	static bool ShouldApplyWarpBlend(IDisplayClusterViewportProxy* ViewportProxy)
	{
		if (ViewportProxy->GetPostRenderSettings_RenderThread().Replace.IsEnabled())
		{
			// When used override texture, disable warp blend
			return false;
		}

		const TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe>& PrjPolicy = ViewportProxy->GetProjectionPolicy_RenderThread();

		// Projection policy must support warp blend op
		return PrjPolicy.IsValid() && PrjPolicy->IsWarpBlendSupported_RenderThread(ViewportProxy);
	}

	static float GetFrameTargetsGamma(const IDisplayClusterViewportManagerProxy* InViewportManagerProxy)
	{
		// Should be replaced by gamma from ViewportManagerProxy or project settings.
		return 2.2f;
	}

};

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureSharePostprocess
//////////////////////////////////////////////////////////////////////////////////////////////// Copyright Epic Games, Inc. All Rights Reserved.
void FTextureSharePostprocess::ShareViewport_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy, const ETextureShareSyncStep InReceiveSyncStep, const EDisplayClusterViewportResourceType InResourceType, const FString& InTextureId, bool bAfterWarpBlend) const
{
	using namespace UE::TextureShare::PostProcess_FrameProxy;

	if (InViewportManagerProxy)
	{
		for (TSharedPtr<IDisplayClusterViewportProxy, ESPMode::ThreadSafe>& ViewportProxyIt : InViewportManagerProxy->GetViewports_RenderThread())
		{
			if (ViewportProxyIt.IsValid() && !ViewportProxyIt->GetContexts_RenderThread().IsEmpty())
			{
				// Get viewport resource type
				EDisplayClusterViewportResourceType ResourceType = InResourceType;
				if (bAfterWarpBlend && ShouldApplyWarpBlend(ViewportProxyIt.Get()))
				{
					TArray<FRHITexture*> Resources;
					if (ViewportProxyIt->GetResources_RenderThread(EDisplayClusterViewportResourceType::AdditionalTargetableResource, Resources) && Resources.Num())
					{
						ResourceType = EDisplayClusterViewportResourceType::AdditionalTargetableResource;
					}
				}

				const bool bMonoscopic = ViewportProxyIt->GetContexts_RenderThread().Num() == 1;

				TArray<FRHITexture*> ViewportResources;
				TArray<FIntRect> ViewportResourceRects;
				if (ViewportProxyIt->GetResourcesWithRects_RenderThread(ResourceType, ViewportResources, ViewportResourceRects))
				{
					for (int32 ContextIndex = 0; ContextIndex < ViewportResources.Num(); ContextIndex++)
					{
						const int32 InGPUIndex = ViewportProxyIt->GetContexts_RenderThread()[ContextIndex].RenderThreadData.GPUIndex;
						const float EngineDisplayGamma = ViewportProxyIt->GetContexts_RenderThread()[ContextIndex].RenderThreadData.EngineDisplayGamma;

						// Gathering UE texture color information
						const FTextureShareColorDesc UEResourceColorDesc(EngineDisplayGamma);

						const ETextureShareEyeType EyeType = bMonoscopic
							? ETextureShareEyeType::Default
							: ((ContextIndex == 0) ? ETextureShareEyeType::StereoLeft : ETextureShareEyeType::StereoRight);

						const FTextureShareCoreViewDesc InViewDesc(ViewportProxyIt->GetId(), EyeType);

						// Execute Send request from the remote process
						ObjectProxy->ShareResource_RenderThread(
							RHICmdList,
							FTextureShareCoreResourceDesc(InTextureId, InViewDesc, ETextureShareTextureOp::Read),
							ViewportResources[ContextIndex],
							UEResourceColorDesc,
							InGPUIndex,
							&ViewportResourceRects[ContextIndex]);
						
						// Execute Receive request from the remote process (Delayed)
						ObjectProxy->ShareResource_RenderThread(
							RHICmdList,
							FTextureShareCoreResourceDesc(InTextureId, InViewDesc, ETextureShareTextureOp::Write, InReceiveSyncStep),
							ViewportResources[ContextIndex],
							UEResourceColorDesc,
							InGPUIndex,
							&ViewportResourceRects[ContextIndex]);
					}
				}
			}
		}
	}
}

void FTextureSharePostprocess::ShareFrame_RenderThread(FRHICommandListImmediate& RHICmdList, const IDisplayClusterViewportManagerProxy* InViewportManagerProxy, const ETextureShareSyncStep InReceiveSyncStep, const EDisplayClusterViewportResourceType InResourceType, const FString& InTextureId) const
{
	using namespace UE::TextureShare::PostProcess_FrameProxy;

	if (InViewportManagerProxy)
	{
		TArray<FRHITexture*> FrameResources;
		TArray<FRHITexture*> AdditionalFrameResources;
		TArray<FIntPoint> TargetOffsets;
		if (InViewportManagerProxy->GetFrameTargets_RenderThread(FrameResources, TargetOffsets, &AdditionalFrameResources))
		{
			TArray<FRHITexture*>& SharedResources = (InResourceType == EDisplayClusterViewportResourceType::AdditionalFrameTargetableResource)
				? AdditionalFrameResources : FrameResources;

			const bool bMonoscopic = SharedResources.Num() == 1;

			const int32 InGPUIndex = -1;
			const float FrameTargetsGamma = GetFrameTargetsGamma(InViewportManagerProxy);

			// Gathering UE texture color information
			const FTextureShareColorDesc FrameTargetColorDesc(FrameTargetsGamma);

			for (int32 ContextIndex = 0; ContextIndex < SharedResources.Num(); ContextIndex++)
			{
				const ETextureShareEyeType EyeType = bMonoscopic
					? ETextureShareEyeType::Default
					: ((ContextIndex == 0) ? ETextureShareEyeType::StereoLeft : ETextureShareEyeType::StereoRight);

				const FTextureShareCoreViewDesc InViewDesc(EyeType);

				// Send
				ObjectProxy->ShareResource_RenderThread(
					RHICmdList,
					FTextureShareCoreResourceDesc(InTextureId, InViewDesc, ETextureShareTextureOp::Read),
					SharedResources[ContextIndex],
					FrameTargetColorDesc,
					InGPUIndex);

				// Receive
				ObjectProxy->ShareResource_RenderThread(
					RHICmdList,
					FTextureShareCoreResourceDesc(InTextureId, InViewDesc, ETextureShareTextureOp::Write, InReceiveSyncStep),
					SharedResources[ContextIndex],
					FrameTargetColorDesc,
					InGPUIndex);
			}
		}
	}
}
