// Copyright Epic Games, Inc. All Rights Reserved.

#include "Projection/TextureShareProjectionPolicy.h"
#include "Misc/TextureShareDisplayClusterStrings.h"

#include "Module/TextureShareDisplayClusterLog.h"
#include "Game/IDisplayClusterGameManager.h"

#include "Render/Viewport/IDisplayClusterViewport.h"
#include "Render/Viewport/IDisplayClusterViewportManager.h"

#include "Containers/TextureShareCoreContainers.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareProjectionPolicy
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareProjectionPolicy::FTextureShareProjectionPolicy(const FString& ProjectionPolicyId, const struct FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
	: FTextureShareProjectionPolicyBase(ProjectionPolicyId, InConfigurationProjectionPolicy)
{
}

FTextureShareProjectionPolicy::~FTextureShareProjectionPolicy()
{
}

//////////////////////////////////////////////////////////////////////////////////////////////
const FString& FTextureShareProjectionPolicy::GetType() const
{
	static const FString Type(UE::TextureShare::DisplayClusterStrings::Projection::TextureShare);

	return Type;
}

bool FTextureShareProjectionPolicy::SetCustomProjection(const TArray<FTextureShareCoreManualProjection>& InProjectionData)
{
	check(IsInGameThread());

	if (InProjectionData.Num() > 2 || (InProjectionData.Num() == 0))
	{
		return false;
	}

	// Sort eyes to [2]: <Mono-Mono> or <Left-Right>
	Projections.Empty();
	Projections.AddDefaulted(2);

	for (const FTextureShareCoreManualProjection& It : InProjectionData)
	{
		switch (It.ViewDesc.EyeType)
		{
		case ETextureShareEyeType::StereoLeft:
			Projections[0] = It;
			break;

		case ETextureShareEyeType::StereoRight:
			Projections[1] = It;
			break;

		case ETextureShareEyeType::Default:
		default:
			if (InProjectionData.Num() != 1)
			{
				// bad input
				return false;
			}

			Projections[0] = Projections[1] = It;
			break;
		}
	}

	return true;
}

//////////////////////////////////////////////////////////////////////////////////////////////
bool FTextureShareProjectionPolicy::HandleStartScene(class IDisplayClusterViewport* InViewport)
{
	return true;
}

void FTextureShareProjectionPolicy::HandleEndScene(class IDisplayClusterViewport* InViewport)
{

}

bool FTextureShareProjectionPolicy::CalculateView(class IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float InNCP, const float InFCP)
{
	check(IsInGameThread());

	if (Projections.Num() == 0)
	{
		// view data not assigned
		return false;
	}

	check((int32)InContextNum < Projections.Num());

	const FTextureShareCoreManualProjection& Src = Projections[InContextNum];

	// Add local rotation specified in config
	switch (Src.ViewLocationType)
	{
	default:
	case ETextureShareViewLocationDataType::Original:
		break;
	case ETextureShareViewLocationDataType::Relative:
		InOutViewLocation += Src.ViewLocation;
		break;
	case ETextureShareViewLocationDataType::Absolute:
		InOutViewLocation = Src.ViewLocation + ViewOffset;
		break;
	case ETextureShareViewLocationDataType::Original_NoViewOffset:
		InOutViewLocation -= ViewOffset;
		break;
	case ETextureShareViewLocationDataType::Relative_NoViewOffset:
		InOutViewLocation -= ViewOffset;
		InOutViewLocation += Src.ViewLocation;
		break;
	case ETextureShareViewLocationDataType::Absolute_NoViewOffset:
		InOutViewLocation = Src.ViewLocation;
		break;
	};

	switch (Src.ViewRotationType)
	{
	default:
	case ETextureShareViewRotationDataType::Original:
		break;
	case ETextureShareViewRotationDataType::Relative:
		InOutViewRotation += Src.ViewRotation;
		break;
	case ETextureShareViewRotationDataType::Absolute:
		InOutViewRotation = Src.ViewRotation;
		break;
	}

	// Store culling data
	NCP = InNCP;
	FCP = InFCP;

	return true;
}

bool FTextureShareProjectionPolicy::GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix)
{
	check(IsInGameThread());

	if (Projections.Num() == 0)
	{
		// view data not assigned
		return false;
	}

	check((int32)InContextNum < Projections.Num());

	const FTextureShareCoreManualProjection& Src = Projections[InContextNum];

	switch (Src.ProjectionType)
	{
	case ETextureShareCoreSceneViewManualProjectionType::Matrix:
		OutPrjMatrix = Src.ProjectionMatrix;
		return true;

	case ETextureShareCoreSceneViewManualProjectionType::FrustumAngles:
		InViewport->CalculateProjectionMatrix(InContextNum, Src.FrustumAngles.Left, Src.FrustumAngles.Right, Src.FrustumAngles.Top, Src.FrustumAngles.Bottom, NCP, FCP, true);
		OutPrjMatrix = InViewport->GetContexts()[InContextNum].ProjectionMatrix;
		return true;

	default:
		break;
	}

	return false;
}
