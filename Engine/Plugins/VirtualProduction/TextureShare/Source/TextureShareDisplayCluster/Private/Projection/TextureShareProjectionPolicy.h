// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Projection/TextureShareProjectionPolicyBase.h"

struct FTextureShareCoreManualProjection;

/**
 * Manual projection policy for TextureShare
 */
class FTextureShareProjectionPolicy
	: public FTextureShareProjectionPolicyBase
{
public:
	FTextureShareProjectionPolicy(const FString& ProjectionPolicyId, const struct FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy);
	virtual ~FTextureShareProjectionPolicy();

public:
	virtual const FString& GetType() const;

	virtual bool HandleStartScene(class IDisplayClusterViewport* InViewport) override;
	virtual void HandleEndScene(class IDisplayClusterViewport* InViewport) override;

	virtual bool CalculateView(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FVector& InOutViewLocation, FRotator& InOutViewRotation, const FVector& ViewOffset, const float WorldToMeters, const float NCP, const float FCP) override;
	virtual bool GetProjectionMatrix(IDisplayClusterViewport* InViewport, const uint32 InContextNum, FMatrix& OutPrjMatrix) override;

	virtual bool IsWarpBlendSupported(const IDisplayClusterViewport* InViewport) const override
	{ return false; }

	virtual bool IsWarpBlendSupported_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy) const override
	{ return false; }

	virtual bool ShouldUseSourceTextureWithMips(const IDisplayClusterViewport* InViewport) const override
	{ return false; }

public:
	bool SetCustomProjection(const TArray<FTextureShareCoreManualProjection>& InProjectionData);

private:
	TArray<FTextureShareCoreManualProjection> Projections;
	
	// Near/far clip planes
	float NCP;
	float FCP;
};
