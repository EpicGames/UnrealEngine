// Copyright Epic Games, Inc. All Rights Reserved.

#include "Projection/TextureShareProjectionPolicyBase.h"
#include "Misc/TextureShareDisplayClusterStrings.h"

#include "IDisplayCluster.h"
#include "Game/IDisplayClusterGameManager.h"

#include "DisplayClusterConfigurationTypes_Base.h"

#include "Render/Viewport/IDisplayClusterViewportManager.h"
#include "Render/Viewport/IDisplayClusterViewport.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareProjectionPolicyBase
//////////////////////////////////////////////////////////////////////////////////////////////
FTextureShareProjectionPolicyBase::FTextureShareProjectionPolicyBase(const FString& InProjectionPolicyId, const FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
	: ProjectionPolicyId(InProjectionPolicyId)
{
	Parameters.Append(InConfigurationProjectionPolicy->Parameters);
}

FTextureShareProjectionPolicyBase::~FTextureShareProjectionPolicyBase()
{
}

bool FTextureShareProjectionPolicyBase::IsEditorOperationMode_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy)
{
	check(IsInRenderingThread());

	if (const bool bIsClusterOperationMode = IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster)
	{
		return false;
	}

	return true;
}

bool FTextureShareProjectionPolicyBase::IsEditorOperationMode(class IDisplayClusterViewport* InViewport)
{
	check(IsInGameThread());

	// Hide spam in logs when configuring VP in editor
	if (const bool bIsClusterOperationMode = IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster)
	{
		return false;
	}

	// Get state from viewport world (UE-114493)
	return InViewport && InViewport->GetConfiguration().IsCurrentWorldHasAnyType(EWorldType::Editor, EWorldType::EditorPreview);
}

bool FTextureShareProjectionPolicyBase::IsConfigurationChanged(const struct FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy) const
{
	if (InConfigurationProjectionPolicy->Parameters.Num() != Parameters.Num())
	{
		return true;
	}

	for (const TPair<FString, FString>& NewParamIt : InConfigurationProjectionPolicy->Parameters)
	{
		const FString* CurrentValue = Parameters.Find(NewParamIt.Key);

		if (CurrentValue==nullptr || CurrentValue->Compare(NewParamIt.Value, ESearchCase::IgnoreCase) != 0)
		{
			return true;
		}
	}

	// Parameters not changed
	return false;
}
