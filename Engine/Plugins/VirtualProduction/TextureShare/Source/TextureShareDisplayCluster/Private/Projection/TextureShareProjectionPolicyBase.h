// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Projection/IDisplayClusterProjectionPolicy.h"

/**
* Base projection policy
*/
class FTextureShareProjectionPolicyBase
	: public IDisplayClusterProjectionPolicy
{
public:
	FTextureShareProjectionPolicyBase(const FString& ProjectionPolicyId, const struct FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy);
	virtual ~FTextureShareProjectionPolicyBase() = 0;

	virtual const FString& GetId() const override
	{
		return ProjectionPolicyId;
	}

	virtual const TMap<FString, FString>& GetParameters() const override
	{
		return Parameters;
	}

	virtual bool IsConfigurationChanged(const struct FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy) const override;

	static bool IsEditorOperationMode(class IDisplayClusterViewport* InViewport);
	static bool IsEditorOperationMode_RenderThread(const IDisplayClusterViewportProxy* InViewportProxy);

private:
	// Added 'Policy' prefix to avoid "... hides class name ..." warnings in child classes
	FString ProjectionPolicyId;
	TMap<FString, FString> Parameters;
};
