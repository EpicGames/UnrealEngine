// Copyright Epic Games, Inc. All Rights Reserved.

#include "Projection/TextureShareProjectionPolicyFactory.h"
#include "Projection/TextureShareProjectionPolicy.h"

#include "Module/TextureShareDisplayClusterLog.h"

#include "DisplayClusterConfigurationTypes.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// FTextureShareProjectionPolicyFactory
//////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> FTextureShareProjectionPolicyFactory::Create(const FString& ProjectionPolicyId, const struct FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy)
{
	check(InConfigurationProjectionPolicy != nullptr);

	UE_LOGF(LogTextureShareDisplayClusterProjection, Verbose, "Instantiating projection policy <%ls> id='%ls'", *InConfigurationProjectionPolicy->Type, *ProjectionPolicyId);

	return  MakeShared<FTextureShareProjectionPolicy, ESPMode::ThreadSafe>(ProjectionPolicyId, InConfigurationProjectionPolicy);
}
