// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Render/Projection/IDisplayClusterProjectionPolicyFactory.h"

/**
 * Projection policy factory for the 'TextureShare' policy
 */
class FTextureShareProjectionPolicyFactory
	: public IDisplayClusterProjectionPolicyFactory
{
public:
	// Constructor for policy
	virtual TSharedPtr<IDisplayClusterProjectionPolicy, ESPMode::ThreadSafe> Create(const FString& ProjectionPolicyId, const struct FDisplayClusterConfigurationProjection* InConfigurationProjectionPolicy) override;
};
