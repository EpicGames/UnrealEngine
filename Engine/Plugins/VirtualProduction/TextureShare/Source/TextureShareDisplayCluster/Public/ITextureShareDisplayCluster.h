// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

/**
 * Public API for nDisplay integration with TextureShare
 */
class ITextureShareDisplayCluster : public IModuleInterface
{
public:
	static constexpr auto ModuleName = TEXT("TextureShareDisplayCluster");

public:
	virtual ~ITextureShareDisplayCluster() = default;

public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton object, loading the module on demand if needed
	 */
	static inline ITextureShareDisplayCluster& Get()
	{
		return FModuleManager::LoadModuleChecked<ITextureShareDisplayCluster>(ITextureShareDisplayCluster::ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(ITextureShareDisplayCluster::ModuleName);
	}

	/**
	 * Returns TextureShare nDisplay API Interface
	 * Put headers in an Internal folder.
	 * This limits visibility to other engine modules, and allows implementations to be changed in hotfixes.
	 */
	virtual class ITextureShareDisplayClusterAPI& GetTextureShareDisplayClusterAPI() = 0;
};
