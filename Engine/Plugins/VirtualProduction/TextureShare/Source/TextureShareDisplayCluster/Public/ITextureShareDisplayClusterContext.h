// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ITextureShareContext.h"

/**
 * TextureShare context for implementing nDisplay integration
 */
class ITextureShareDisplayClusterContext
	: public ITextureShareContext
{
public:
	virtual ~ITextureShareDisplayClusterContext() = default;

	/** A quick and dirty way to determine which TS data (sub)class this is. */
	virtual FName GetRTTI() const { return TEXT("TextureShareDisplayCluster"); }
};
