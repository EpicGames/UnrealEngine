// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

[SupportedPlatforms("Win64")]
public class TextureShareDisplayCluster : ModuleRules
{
	public TextureShareDisplayCluster(ReadOnlyTargetRules Target) : base(Target)
	{
		// Internal dependency (debug log purpose)
		string EngineDir = Path.GetFullPath(Target.RelativeEnginePath);
		PrivateIncludePaths.Add(Path.Combine(EngineDir, "Plugins", "VirtualProduction", "TextureShare", "Source", "TextureShareCore", "Private"));
		PrivateIncludePaths.Add(Path.Combine(EngineDir, "Plugins", "VirtualProduction", "TextureShare", "Source", "TextureShare", "Private"));

		// Show more log for internal sync processes
		bool bEnableExtraDebugLog = false;
		if (bEnableExtraDebugLog)
		{
			//Show log in SDK-for Debug and DebugGame builds
			PublicDefinitions.Add("TEXTURESHAREDISPLAYCLUSTER_DEBUGLOG=1");
		}
		else
		{
			PublicDefinitions.Add("TEXTURESHAREDISPLAYCLUSTER_DEBUGLOG=0");
		}

		// List of public dependency module names (no path needed) (automatically does the private/public include). These are modules that are required by our public source files.
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DisplayCluster",
				"DisplayClusterShaders",
				"DisplayClusterConfiguration",
			});

		// List of private dependency module names.  These are modules that our private code depends on but nothing in our public include files depend on.
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"RHI",
				"Renderer",
				"RenderCore",
				"TextureShare",
				"TextureShareCore",
			});
	}
}
